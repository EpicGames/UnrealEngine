; Add hardware devices here to detect them at runtime using FInputDeviceScope::GetCurrent
; @see FRawInputWindows::SendControllerEvents()

; [InputPlatformSettings_Windows InputPlatformSettings]
; +HardwareDevices=(InputClassName="RawInput",HardwareDeviceIdentifier="<YourDeviceName>")