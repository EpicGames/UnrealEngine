{
    "FileVersion" : 1,
	
	"FriendlyName" : "Windows RawInput",
	"Version" : 1,
	"VersionName" : "1.0",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"Description" : "RawInput provides an interface to receive input from  Flight Sticks, Steering Wheels, and other non-XInput supported devices in Windows.",
	"Category" : "Input Devices",
	"EnabledByDefault" : false,
	"DeprecatedEngineVersion": "5.8",
    "Modules" :
	[
		{
			"Name" : "RawInput",
			"Type" : "Runtime",
			"PlatformAllowList" :
			[
				"Win64"
			]
		}
	]
}