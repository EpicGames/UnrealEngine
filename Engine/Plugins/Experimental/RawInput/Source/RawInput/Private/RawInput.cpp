// Copyright Epic Games, Inc. All Rights Reserved.

#include "RawInput.h"
#include "RawInputFunctionLibrary.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"

#include "Null/NullPlatformApplicationMisc.h"

#if PLATFORM_WINDOWS
	#include "Windows/RawInputWindows.h"
#endif

#define LOCTEXT_NAMESPACE "RawInputPlugin"

// Generic USB controller (Wheels, flight sticks etc. These require the rawinput plugin to be enabled)
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis1( "GenericUSBController_Axis1" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis2( "GenericUSBController_Axis2" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis3( "GenericUSBController_Axis3" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis4( "GenericUSBController_Axis4" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis5( "GenericUSBController_Axis5" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis6( "GenericUSBController_Axis6" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis7( "GenericUSBController_Axis7" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis8( "GenericUSBController_Axis8" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis9( "GenericUSBController_Axis9" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis10( "GenericUSBController_Axis10" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis11( "GenericUSBController_Axis11" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis12( "GenericUSBController_Axis12" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis13( "GenericUSBController_Axis13" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis14( "GenericUSBController_Axis14" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis15( "GenericUSBController_Axis15" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis16( "GenericUSBController_Axis16" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis17( "GenericUSBController_Axis17" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis18( "GenericUSBController_Axis18" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis19( "GenericUSBController_Axis19" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis20( "GenericUSBController_Axis20" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis21( "GenericUSBController_Axis21" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis22( "GenericUSBController_Axis22" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis23( "GenericUSBController_Axis23" );
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Axis24( "GenericUSBController_Axis24" );

const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button1("GenericUSBController_Button1");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button2("GenericUSBController_Button2");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button3("GenericUSBController_Button3");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button4("GenericUSBController_Button4");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button5("GenericUSBController_Button5");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button6("GenericUSBController_Button6");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button7("GenericUSBController_Button7");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button8("GenericUSBController_Button8");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button9("GenericUSBController_Button9");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button10("GenericUSBController_Button10");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button11("GenericUSBController_Button11");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button12("GenericUSBController_Button12");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button13("GenericUSBController_Button13");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button14("GenericUSBController_Button14");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button15("GenericUSBController_Button15");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button16("GenericUSBController_Button16");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button17("GenericUSBController_Button17");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button18("GenericUSBController_Button18");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button19("GenericUSBController_Button19");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button20("GenericUSBController_Button20");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button21("GenericUSBController_Button21");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button22("GenericUSBController_Button22");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button23("GenericUSBController_Button23");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button24("GenericUSBController_Button24");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button25("GenericUSBController_Button25");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button26("GenericUSBController_Button26");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button27("GenericUSBController_Button27");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button28("GenericUSBController_Button28");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button29("GenericUSBController_Button29");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button30("GenericUSBController_Button30");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button31("GenericUSBController_Button31");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button32("GenericUSBController_Button32");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button33("GenericUSBController_Button33");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button34("GenericUSBController_Button34");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button35("GenericUSBController_Button35");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button36("GenericUSBController_Button36");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button37("GenericUSBController_Button37");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button38("GenericUSBController_Button38");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button39("GenericUSBController_Button39");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button40("GenericUSBController_Button40");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button41("GenericUSBController_Button41");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button42("GenericUSBController_Button42");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button43("GenericUSBController_Button43");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button44("GenericUSBController_Button44");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button45("GenericUSBController_Button45");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button46("GenericUSBController_Button46");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button47("GenericUSBController_Button47");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button48("GenericUSBController_Button48");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button49("GenericUSBController_Button49");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button50("GenericUSBController_Button50");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button51("GenericUSBController_Button51");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button52("GenericUSBController_Button52");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button53("GenericUSBController_Button53");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button54("GenericUSBController_Button54");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button55("GenericUSBController_Button55");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button56("GenericUSBController_Button56");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button57("GenericUSBController_Button57");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button58("GenericUSBController_Button58");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button59("GenericUSBController_Button59");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button60("GenericUSBController_Button60");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button61("GenericUSBController_Button61");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button62("GenericUSBController_Button62");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button63("GenericUSBController_Button63");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button64("GenericUSBController_Button64");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button65("GenericUSBController_Button65");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button66("GenericUSBController_Button66");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button67("GenericUSBController_Button67");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button68("GenericUSBController_Button68");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button69("GenericUSBController_Button69");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button70("GenericUSBController_Button70");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button71("GenericUSBController_Button71");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button72("GenericUSBController_Button72");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button73("GenericUSBController_Button73");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button74("GenericUSBController_Button74");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button75("GenericUSBController_Button75");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button76("GenericUSBController_Button76");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button77("GenericUSBController_Button77");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button78("GenericUSBController_Button78");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button79("GenericUSBController_Button79");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button80("GenericUSBController_Button80");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button81("GenericUSBController_Button81");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button82("GenericUSBController_Button82");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button83("GenericUSBController_Button83");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button84("GenericUSBController_Button84");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button85("GenericUSBController_Button85");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button86("GenericUSBController_Button86");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button87("GenericUSBController_Button87");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button88("GenericUSBController_Button88");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button89("GenericUSBController_Button89");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button90("GenericUSBController_Button90");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button91("GenericUSBController_Button91");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button92("GenericUSBController_Button92");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button93("GenericUSBController_Button93");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button94("GenericUSBController_Button94");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button95("GenericUSBController_Button95");
const FGamepadKeyNames::Type FRawInputKeyNames::GenericUSBController_Button96("GenericUSBController_Button96");

// USB controller (Wheels, flight stick etc)
const FKey FRawInputKeys::GenericUSBController_Axis1(FRawInputKeyNames::GenericUSBController_Axis1);
const FKey FRawInputKeys::GenericUSBController_Axis2(FRawInputKeyNames::GenericUSBController_Axis2);
const FKey FRawInputKeys::GenericUSBController_Axis3(FRawInputKeyNames::GenericUSBController_Axis3);
const FKey FRawInputKeys::GenericUSBController_Axis4(FRawInputKeyNames::GenericUSBController_Axis4);
const FKey FRawInputKeys::GenericUSBController_Axis5(FRawInputKeyNames::GenericUSBController_Axis5);
const FKey FRawInputKeys::GenericUSBController_Axis6(FRawInputKeyNames::GenericUSBController_Axis6);
const FKey FRawInputKeys::GenericUSBController_Axis7(FRawInputKeyNames::GenericUSBController_Axis7);
const FKey FRawInputKeys::GenericUSBController_Axis8(FRawInputKeyNames::GenericUSBController_Axis8);
const FKey FRawInputKeys::GenericUSBController_Axis9(FRawInputKeyNames::GenericUSBController_Axis9);
const FKey FRawInputKeys::GenericUSBController_Axis10(FRawInputKeyNames::GenericUSBController_Axis10);
const FKey FRawInputKeys::GenericUSBController_Axis11(FRawInputKeyNames::GenericUSBController_Axis11);
const FKey FRawInputKeys::GenericUSBController_Axis12(FRawInputKeyNames::GenericUSBController_Axis12);
const FKey FRawInputKeys::GenericUSBController_Axis13(FRawInputKeyNames::GenericUSBController_Axis13);
const FKey FRawInputKeys::GenericUSBController_Axis14(FRawInputKeyNames::GenericUSBController_Axis14);
const FKey FRawInputKeys::GenericUSBController_Axis15(FRawInputKeyNames::GenericUSBController_Axis15);
const FKey FRawInputKeys::GenericUSBController_Axis16(FRawInputKeyNames::GenericUSBController_Axis16);
const FKey FRawInputKeys::GenericUSBController_Axis17(FRawInputKeyNames::GenericUSBController_Axis17);
const FKey FRawInputKeys::GenericUSBController_Axis18(FRawInputKeyNames::GenericUSBController_Axis18);
const FKey FRawInputKeys::GenericUSBController_Axis19(FRawInputKeyNames::GenericUSBController_Axis19);
const FKey FRawInputKeys::GenericUSBController_Axis20(FRawInputKeyNames::GenericUSBController_Axis20);
const FKey FRawInputKeys::GenericUSBController_Axis21(FRawInputKeyNames::GenericUSBController_Axis21);
const FKey FRawInputKeys::GenericUSBController_Axis22(FRawInputKeyNames::GenericUSBController_Axis22);
const FKey FRawInputKeys::GenericUSBController_Axis23(FRawInputKeyNames::GenericUSBController_Axis23);
const FKey FRawInputKeys::GenericUSBController_Axis24(FRawInputKeyNames::GenericUSBController_Axis24);

const FKey FRawInputKeys::GenericUSBController_Button1(FRawInputKeyNames::GenericUSBController_Button1);
const FKey FRawInputKeys::GenericUSBController_Button2(FRawInputKeyNames::GenericUSBController_Button2);
const FKey FRawInputKeys::GenericUSBController_Button3(FRawInputKeyNames::GenericUSBController_Button3);
const FKey FRawInputKeys::GenericUSBController_Button4(FRawInputKeyNames::GenericUSBController_Button4);
const FKey FRawInputKeys::GenericUSBController_Button5(FRawInputKeyNames::GenericUSBController_Button5);
const FKey FRawInputKeys::GenericUSBController_Button6(FRawInputKeyNames::GenericUSBController_Button6);
const FKey FRawInputKeys::GenericUSBController_Button7(FRawInputKeyNames::GenericUSBController_Button7);
const FKey FRawInputKeys::GenericUSBController_Button8(FRawInputKeyNames::GenericUSBController_Button8);
const FKey FRawInputKeys::GenericUSBController_Button9(FRawInputKeyNames::GenericUSBController_Button9);
const FKey FRawInputKeys::GenericUSBController_Button10(FRawInputKeyNames::GenericUSBController_Button10);
const FKey FRawInputKeys::GenericUSBController_Button11(FRawInputKeyNames::GenericUSBController_Button11);
const FKey FRawInputKeys::GenericUSBController_Button12(FRawInputKeyNames::GenericUSBController_Button12);
const FKey FRawInputKeys::GenericUSBController_Button13(FRawInputKeyNames::GenericUSBController_Button13);
const FKey FRawInputKeys::GenericUSBController_Button14(FRawInputKeyNames::GenericUSBController_Button14);
const FKey FRawInputKeys::GenericUSBController_Button15(FRawInputKeyNames::GenericUSBController_Button15);
const FKey FRawInputKeys::GenericUSBController_Button16(FRawInputKeyNames::GenericUSBController_Button16);
const FKey FRawInputKeys::GenericUSBController_Button17(FRawInputKeyNames::GenericUSBController_Button17);
const FKey FRawInputKeys::GenericUSBController_Button18(FRawInputKeyNames::GenericUSBController_Button18);
const FKey FRawInputKeys::GenericUSBController_Button19(FRawInputKeyNames::GenericUSBController_Button19);
const FKey FRawInputKeys::GenericUSBController_Button20(FRawInputKeyNames::GenericUSBController_Button20);
const FKey FRawInputKeys::GenericUSBController_Button21(FRawInputKeyNames::GenericUSBController_Button21);
const FKey FRawInputKeys::GenericUSBController_Button22(FRawInputKeyNames::GenericUSBController_Button22);
const FKey FRawInputKeys::GenericUSBController_Button23(FRawInputKeyNames::GenericUSBController_Button23);
const FKey FRawInputKeys::GenericUSBController_Button24(FRawInputKeyNames::GenericUSBController_Button24);
const FKey FRawInputKeys::GenericUSBController_Button25(FRawInputKeyNames::GenericUSBController_Button25);
const FKey FRawInputKeys::GenericUSBController_Button26(FRawInputKeyNames::GenericUSBController_Button26);
const FKey FRawInputKeys::GenericUSBController_Button27(FRawInputKeyNames::GenericUSBController_Button27);
const FKey FRawInputKeys::GenericUSBController_Button28(FRawInputKeyNames::GenericUSBController_Button28);
const FKey FRawInputKeys::GenericUSBController_Button29(FRawInputKeyNames::GenericUSBController_Button29);
const FKey FRawInputKeys::GenericUSBController_Button30(FRawInputKeyNames::GenericUSBController_Button30);
const FKey FRawInputKeys::GenericUSBController_Button31(FRawInputKeyNames::GenericUSBController_Button31);
const FKey FRawInputKeys::GenericUSBController_Button32(FRawInputKeyNames::GenericUSBController_Button32);
const FKey FRawInputKeys::GenericUSBController_Button33(FRawInputKeyNames::GenericUSBController_Button33);
const FKey FRawInputKeys::GenericUSBController_Button34(FRawInputKeyNames::GenericUSBController_Button34);
const FKey FRawInputKeys::GenericUSBController_Button35(FRawInputKeyNames::GenericUSBController_Button35);
const FKey FRawInputKeys::GenericUSBController_Button36(FRawInputKeyNames::GenericUSBController_Button36);
const FKey FRawInputKeys::GenericUSBController_Button37(FRawInputKeyNames::GenericUSBController_Button37);
const FKey FRawInputKeys::GenericUSBController_Button38(FRawInputKeyNames::GenericUSBController_Button38);
const FKey FRawInputKeys::GenericUSBController_Button39(FRawInputKeyNames::GenericUSBController_Button39);
const FKey FRawInputKeys::GenericUSBController_Button40(FRawInputKeyNames::GenericUSBController_Button40);
const FKey FRawInputKeys::GenericUSBController_Button41(FRawInputKeyNames::GenericUSBController_Button41);
const FKey FRawInputKeys::GenericUSBController_Button42(FRawInputKeyNames::GenericUSBController_Button42);
const FKey FRawInputKeys::GenericUSBController_Button43(FRawInputKeyNames::GenericUSBController_Button43);
const FKey FRawInputKeys::GenericUSBController_Button44(FRawInputKeyNames::GenericUSBController_Button44);
const FKey FRawInputKeys::GenericUSBController_Button45(FRawInputKeyNames::GenericUSBController_Button45);
const FKey FRawInputKeys::GenericUSBController_Button46(FRawInputKeyNames::GenericUSBController_Button46);
const FKey FRawInputKeys::GenericUSBController_Button47(FRawInputKeyNames::GenericUSBController_Button47);
const FKey FRawInputKeys::GenericUSBController_Button48(FRawInputKeyNames::GenericUSBController_Button48);
const FKey FRawInputKeys::GenericUSBController_Button49(FRawInputKeyNames::GenericUSBController_Button49);
const FKey FRawInputKeys::GenericUSBController_Button50(FRawInputKeyNames::GenericUSBController_Button50);
const FKey FRawInputKeys::GenericUSBController_Button51(FRawInputKeyNames::GenericUSBController_Button51);
const FKey FRawInputKeys::GenericUSBController_Button52(FRawInputKeyNames::GenericUSBController_Button52);
const FKey FRawInputKeys::GenericUSBController_Button53(FRawInputKeyNames::GenericUSBController_Button53);
const FKey FRawInputKeys::GenericUSBController_Button54(FRawInputKeyNames::GenericUSBController_Button54);
const FKey FRawInputKeys::GenericUSBController_Button55(FRawInputKeyNames::GenericUSBController_Button55);
const FKey FRawInputKeys::GenericUSBController_Button56(FRawInputKeyNames::GenericUSBController_Button56);
const FKey FRawInputKeys::GenericUSBController_Button57(FRawInputKeyNames::GenericUSBController_Button57);
const FKey FRawInputKeys::GenericUSBController_Button58(FRawInputKeyNames::GenericUSBController_Button58);
const FKey FRawInputKeys::GenericUSBController_Button59(FRawInputKeyNames::GenericUSBController_Button59);
const FKey FRawInputKeys::GenericUSBController_Button60(FRawInputKeyNames::GenericUSBController_Button60);
const FKey FRawInputKeys::GenericUSBController_Button61(FRawInputKeyNames::GenericUSBController_Button61);
const FKey FRawInputKeys::GenericUSBController_Button62(FRawInputKeyNames::GenericUSBController_Button62);
const FKey FRawInputKeys::GenericUSBController_Button63(FRawInputKeyNames::GenericUSBController_Button63);
const FKey FRawInputKeys::GenericUSBController_Button64(FRawInputKeyNames::GenericUSBController_Button64);
const FKey FRawInputKeys::GenericUSBController_Button65(FRawInputKeyNames::GenericUSBController_Button65);
const FKey FRawInputKeys::GenericUSBController_Button66(FRawInputKeyNames::GenericUSBController_Button66);
const FKey FRawInputKeys::GenericUSBController_Button67(FRawInputKeyNames::GenericUSBController_Button67);
const FKey FRawInputKeys::GenericUSBController_Button68(FRawInputKeyNames::GenericUSBController_Button68);
const FKey FRawInputKeys::GenericUSBController_Button69(FRawInputKeyNames::GenericUSBController_Button69);
const FKey FRawInputKeys::GenericUSBController_Button70(FRawInputKeyNames::GenericUSBController_Button70);
const FKey FRawInputKeys::GenericUSBController_Button71(FRawInputKeyNames::GenericUSBController_Button71);
const FKey FRawInputKeys::GenericUSBController_Button72(FRawInputKeyNames::GenericUSBController_Button72);
const FKey FRawInputKeys::GenericUSBController_Button73(FRawInputKeyNames::GenericUSBController_Button73);
const FKey FRawInputKeys::GenericUSBController_Button74(FRawInputKeyNames::GenericUSBController_Button74);
const FKey FRawInputKeys::GenericUSBController_Button75(FRawInputKeyNames::GenericUSBController_Button75);
const FKey FRawInputKeys::GenericUSBController_Button76(FRawInputKeyNames::GenericUSBController_Button76);
const FKey FRawInputKeys::GenericUSBController_Button77(FRawInputKeyNames::GenericUSBController_Button77);
const FKey FRawInputKeys::GenericUSBController_Button78(FRawInputKeyNames::GenericUSBController_Button78);
const FKey FRawInputKeys::GenericUSBController_Button79(FRawInputKeyNames::GenericUSBController_Button79);
const FKey FRawInputKeys::GenericUSBController_Button80(FRawInputKeyNames::GenericUSBController_Button80);
const FKey FRawInputKeys::GenericUSBController_Button81(FRawInputKeyNames::GenericUSBController_Button81);
const FKey FRawInputKeys::GenericUSBController_Button82(FRawInputKeyNames::GenericUSBController_Button82);
const FKey FRawInputKeys::GenericUSBController_Button83(FRawInputKeyNames::GenericUSBController_Button83);
const FKey FRawInputKeys::GenericUSBController_Button84(FRawInputKeyNames::GenericUSBController_Button84);
const FKey FRawInputKeys::GenericUSBController_Button85(FRawInputKeyNames::GenericUSBController_Button85);
const FKey FRawInputKeys::GenericUSBController_Button86(FRawInputKeyNames::GenericUSBController_Button86);
const FKey FRawInputKeys::GenericUSBController_Button87(FRawInputKeyNames::GenericUSBController_Button87);
const FKey FRawInputKeys::GenericUSBController_Button88(FRawInputKeyNames::GenericUSBController_Button88);
const FKey FRawInputKeys::GenericUSBController_Button89(FRawInputKeyNames::GenericUSBController_Button89);
const FKey FRawInputKeys::GenericUSBController_Button90(FRawInputKeyNames::GenericUSBController_Button90);
const FKey FRawInputKeys::GenericUSBController_Button91(FRawInputKeyNames::GenericUSBController_Button91);
const FKey FRawInputKeys::GenericUSBController_Button92(FRawInputKeyNames::GenericUSBController_Button92);
const FKey FRawInputKeys::GenericUSBController_Button93(FRawInputKeyNames::GenericUSBController_Button93);
const FKey FRawInputKeys::GenericUSBController_Button94(FRawInputKeyNames::GenericUSBController_Button94);
const FKey FRawInputKeys::GenericUSBController_Button95(FRawInputKeyNames::GenericUSBController_Button95);
const FKey FRawInputKeys::GenericUSBController_Button96(FRawInputKeyNames::GenericUSBController_Button96);

IRawInput::IRawInput(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
	: MessageHandler(InMessageHandler)
	, LastAssignedInputHandle(0)
{
};

TSharedPtr< class IInputDevice > FRawInputPlugin::CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
{
	// Don't create an input device when running offscreen
	const bool bRunningOffscreen = FNullPlatformApplicationMisc::IsUsingNullApplication();
	if (bRunningOffscreen)
	{
		return nullptr;
	}

	RawInputDevice = MakeShareable( new FPlatformRawInput(InMessageHandler) ); 
	return RawInputDevice;
}

void FRawInputPlugin::StartupModule()
{
	IInputDeviceModule::StartupModule();

	const FName NAME_GenericUSBController(TEXT("GenericUSBController"));

	// Generic USB Controllers (Wheel, Flightstick etc.)
	EKeys::AddMenuCategoryDisplayInfo(NAME_GenericUSBController, LOCTEXT("GenericUSBControllerSubCateogry", "GenericUSBController"), TEXT("GraphEditor.KeyEvent_16x"));

	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis1, LOCTEXT("GenericUSBController_Axis1", "GenericUSBController Axis 1"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis2, LOCTEXT("GenericUSBController_Axis2", "GenericUSBController Axis 2"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis3, LOCTEXT("GenericUSBController_Axis3", "GenericUSBController Axis 3"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis4, LOCTEXT("GenericUSBController_Axis4", "GenericUSBController Axis 4"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis5, LOCTEXT("GenericUSBController_Axis5", "GenericUSBController Axis 5"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis6, LOCTEXT("GenericUSBController_Axis6", "GenericUSBController Axis 6"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis7, LOCTEXT("GenericUSBController_Axis7", "GenericUSBController Axis 7"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis8, LOCTEXT("GenericUSBController_Axis8", "GenericUSBController Axis 8"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis9, LOCTEXT("GenericUSBController_Axis9", "GenericUSBController Axis 9"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis10, LOCTEXT("GenericUSBController_Axis10", "GenericUSBController Axis 10"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis11, LOCTEXT("GenericUSBController_Axis11", "GenericUSBController Axis 11"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis12, LOCTEXT("GenericUSBController_Axis12", "GenericUSBController Axis 12"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis13, LOCTEXT("GenericUSBController_Axis13", "GenericUSBController Axis 13"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis14, LOCTEXT("GenericUSBController_Axis14", "GenericUSBController Axis 14"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis15, LOCTEXT("GenericUSBController_Axis15", "GenericUSBController Axis 15"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis16, LOCTEXT("GenericUSBController_Axis16", "GenericUSBController Axis 16"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis17, LOCTEXT("GenericUSBController_Axis17", "GenericUSBController Axis 17"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis18, LOCTEXT("GenericUSBController_Axis18", "GenericUSBController Axis 18"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis19, LOCTEXT("GenericUSBController_Axis19", "GenericUSBController Axis 19"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis20, LOCTEXT("GenericUSBController_Axis20", "GenericUSBController Axis 20"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis21, LOCTEXT("GenericUSBController_Axis21", "GenericUSBController Axis 21"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis22, LOCTEXT("GenericUSBController_Axis22", "GenericUSBController Axis 22"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis23, LOCTEXT("GenericUSBController_Axis23", "GenericUSBController Axis 23"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Axis24, LOCTEXT("GenericUSBController_Axis24", "GenericUSBController Axis 24"), FKeyDetails::GamepadKey, NAME_GenericUSBController));

	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button1, LOCTEXT("GenericUSBController_Button1", "GenericUSBController Button 1"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button2, LOCTEXT("GenericUSBController_Button2", "GenericUSBController Button 2"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button3, LOCTEXT("GenericUSBController_Button3", "GenericUSBController Button 3"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button4, LOCTEXT("GenericUSBController_Button4", "GenericUSBController Button 4"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button5, LOCTEXT("GenericUSBController_Button5", "GenericUSBController Button 5"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button6, LOCTEXT("GenericUSBController_Button6", "GenericUSBController Button 6"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button7, LOCTEXT("GenericUSBController_Button7", "GenericUSBController Button 7"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button8, LOCTEXT("GenericUSBController_Button8", "GenericUSBController Button 8"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button9, LOCTEXT("GenericUSBController_Button9", "GenericUSBController Button 9"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button10, LOCTEXT("GenericUSBController_Button10", "GenericUSBController Button 10"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button11, LOCTEXT("GenericUSBController_Button11", "GenericUSBController Button 11"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button12, LOCTEXT("GenericUSBController_Button12", "GenericUSBController Button 12"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button13, LOCTEXT("GenericUSBController_Button13", "GenericUSBController Button 13"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button14, LOCTEXT("GenericUSBController_Button14", "GenericUSBController Button 14"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button15, LOCTEXT("GenericUSBController_Button15", "GenericUSBController Button 15"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button16, LOCTEXT("GenericUSBController_Button16", "GenericUSBController Button 16"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button17, LOCTEXT("GenericUSBController_Button17", "GenericUSBController Button 17"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button18, LOCTEXT("GenericUSBController_Button18", "GenericUSBController Button 18"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button19, LOCTEXT("GenericUSBController_Button19", "GenericUSBController Button 19"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button20, LOCTEXT("GenericUSBController_Button20", "GenericUSBController Button 20"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button21, LOCTEXT("GenericUSBController_Button21", "GenericUSBController Button 21"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button22, LOCTEXT("GenericUSBController_Button22", "GenericUSBController Button 22"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button23, LOCTEXT("GenericUSBController_Button23", "GenericUSBController Button 23"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button24, LOCTEXT("GenericUSBController_Button24", "GenericUSBController Button 24"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button25, LOCTEXT("GenericUSBController_Button25", "GenericUSBController Button 25"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button26, LOCTEXT("GenericUSBController_Button26", "GenericUSBController Button 26"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button27, LOCTEXT("GenericUSBController_Button27", "GenericUSBController Button 27"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button28, LOCTEXT("GenericUSBController_Button28", "GenericUSBController Button 28"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button29, LOCTEXT("GenericUSBController_Button29", "GenericUSBController Button 29"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button30, LOCTEXT("GenericUSBController_Button30", "GenericUSBController Button 30"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button31, LOCTEXT("GenericUSBController_Button31", "GenericUSBController Button 31"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button32, LOCTEXT("GenericUSBController_Button32", "GenericUSBController Button 32"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button33, LOCTEXT("GenericUSBController_Button33", "GenericUSBController Button 33"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button34, LOCTEXT("GenericUSBController_Button34", "GenericUSBController Button 34"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button35, LOCTEXT("GenericUSBController_Button35", "GenericUSBController Button 35"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button36, LOCTEXT("GenericUSBController_Button36", "GenericUSBController Button 36"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button37, LOCTEXT("GenericUSBController_Button37", "GenericUSBController Button 37"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button38, LOCTEXT("GenericUSBController_Button38", "GenericUSBController Button 38"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button39, LOCTEXT("GenericUSBController_Button39", "GenericUSBController Button 39"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button40, LOCTEXT("GenericUSBController_Button40", "GenericUSBController Button 40"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button41, LOCTEXT("GenericUSBController_Button41", "GenericUSBController Button 41"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button42, LOCTEXT("GenericUSBController_Button42", "GenericUSBController Button 42"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button43, LOCTEXT("GenericUSBController_Button43", "GenericUSBController Button 43"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button44, LOCTEXT("GenericUSBController_Button44", "GenericUSBController Button 44"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button45, LOCTEXT("GenericUSBController_Button45", "GenericUSBController Button 45"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button46, LOCTEXT("GenericUSBController_Button46", "GenericUSBController Button 46"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button47, LOCTEXT("GenericUSBController_Button47", "GenericUSBController Button 47"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button48, LOCTEXT("GenericUSBController_Button48", "GenericUSBController Button 48"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button49, LOCTEXT("GenericUSBController_Button49", "GenericUSBController Button 49"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button50, LOCTEXT("GenericUSBController_Button50", "GenericUSBController Button 50"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button51, LOCTEXT("GenericUSBController_Button51", "GenericUSBController Button 51"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button52, LOCTEXT("GenericUSBController_Button52", "GenericUSBController Button 52"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button53, LOCTEXT("GenericUSBController_Button53", "GenericUSBController Button 53"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button54, LOCTEXT("GenericUSBController_Button54", "GenericUSBController Button 54"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button55, LOCTEXT("GenericUSBController_Button55", "GenericUSBController Button 55"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button56, LOCTEXT("GenericUSBController_Button56", "GenericUSBController Button 56"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button57, LOCTEXT("GenericUSBController_Button57", "GenericUSBController Button 57"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button58, LOCTEXT("GenericUSBController_Button58", "GenericUSBController Button 58"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button59, LOCTEXT("GenericUSBController_Button59", "GenericUSBController Button 59"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button60, LOCTEXT("GenericUSBController_Button60", "GenericUSBController Button 60"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button61, LOCTEXT("GenericUSBController_Button61", "GenericUSBController Button 61"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button62, LOCTEXT("GenericUSBController_Button62", "GenericUSBController Button 62"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button63, LOCTEXT("GenericUSBController_Button63", "GenericUSBController Button 63"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button64, LOCTEXT("GenericUSBController_Button64", "GenericUSBController Button 64"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button65, LOCTEXT("GenericUSBController_Button65", "GenericUSBController Button 65"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button66, LOCTEXT("GenericUSBController_Button66", "GenericUSBController Button 66"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button67, LOCTEXT("GenericUSBController_Button67", "GenericUSBController Button 67"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button68, LOCTEXT("GenericUSBController_Button68", "GenericUSBController Button 68"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button69, LOCTEXT("GenericUSBController_Button69", "GenericUSBController Button 69"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button70, LOCTEXT("GenericUSBController_Button70", "GenericUSBController Button 70"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button71, LOCTEXT("GenericUSBController_Button71", "GenericUSBController Button 71"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button72, LOCTEXT("GenericUSBController_Button72", "GenericUSBController Button 72"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button73, LOCTEXT("GenericUSBController_Button73", "GenericUSBController Button 73"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button74, LOCTEXT("GenericUSBController_Button74", "GenericUSBController Button 74"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button75, LOCTEXT("GenericUSBController_Button75", "GenericUSBController Button 75"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button76, LOCTEXT("GenericUSBController_Button76", "GenericUSBController Button 76"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button77, LOCTEXT("GenericUSBController_Button77", "GenericUSBController Button 77"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button78, LOCTEXT("GenericUSBController_Button78", "GenericUSBController Button 78"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button79, LOCTEXT("GenericUSBController_Button79", "GenericUSBController Button 79"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button80, LOCTEXT("GenericUSBController_Button80", "GenericUSBController Button 80"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button81, LOCTEXT("GenericUSBController_Button81", "GenericUSBController Button 81"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button82, LOCTEXT("GenericUSBController_Button82", "GenericUSBController Button 82"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button83, LOCTEXT("GenericUSBController_Button83", "GenericUSBController Button 83"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button84, LOCTEXT("GenericUSBController_Button84", "GenericUSBController Button 84"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button85, LOCTEXT("GenericUSBController_Button85", "GenericUSBController Button 85"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button86, LOCTEXT("GenericUSBController_Button86", "GenericUSBController Button 86"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button87, LOCTEXT("GenericUSBController_Button87", "GenericUSBController Button 87"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button88, LOCTEXT("GenericUSBController_Button88", "GenericUSBController Button 88"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button89, LOCTEXT("GenericUSBController_Button89", "GenericUSBController Button 89"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button90, LOCTEXT("GenericUSBController_Button90", "GenericUSBController Button 90"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button91, LOCTEXT("GenericUSBController_Button91", "GenericUSBController Button 91"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button92, LOCTEXT("GenericUSBController_Button92", "GenericUSBController Button 92"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button93, LOCTEXT("GenericUSBController_Button93", "GenericUSBController Button 93"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button94, LOCTEXT("GenericUSBController_Button94", "GenericUSBController Button 94"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button95, LOCTEXT("GenericUSBController_Button95", "GenericUSBController Button 95"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
	EKeys::AddKey(FKeyDetails(FRawInputKeys::GenericUSBController_Button96, LOCTEXT("GenericUSBController_Button96", "GenericUSBController Button 96"), FKeyDetails::GamepadKey, NAME_GenericUSBController));
}

IMPLEMENT_MODULE( FRawInputPlugin, RawInput)

#undef LOCTEXT_NAMESPACE
