// Copyright Epic Games, Inc. All Rights Reserved.

#include "RawInputFunctionLibrary.h"
#if PLATFORM_WINDOWS
	#include "Windows/RawInputWindows.h"
#endif

TArray<FRegisteredDeviceInfo> URawInputFunctionLibrary::GetRegisteredDevices()
{
	TArray<FRegisteredDeviceInfo> RegisteredDevices;

#if PLATFORM_WINDOWS
	FRawInputWindows* RawInput = static_cast<FRawInputWindows*>(static_cast<FRawInputPlugin*>(&FRawInputPlugin::Get())->GetRawInputDevice().Get());

	RegisteredDevices.Reserve(RawInput->RegisteredDeviceList.Num());
	for (const TPair<int32, FRawWindowsDeviceEntry>& RegisteredDevice : RawInput->RegisteredDeviceList)
	{
		const FRawWindowsDeviceEntry& DeviceEntry = RegisteredDevice.Value;
		if (DeviceEntry.bIsConnected)
		{
			RegisteredDevices.Add(RawInput->GetDeviceInfo(RegisteredDevice.Key));
		}
	}

#endif

	return RegisteredDevices;
}