// Copyright Epic Games, Inc. All Rights Reserved.

#include "RawInputSettings.h"
#include "RawInputFunctionLibrary.h"

#if PLATFORM_WINDOWS
	#include "Windows/RawInputWindows.h"
#endif

FRawInputDeviceConfiguration::FRawInputDeviceConfiguration()
{
	ButtonProperties.AddDefaulted(MAX_NUM_CONTROLLER_BUTTONS);
	AxisProperties.AddDefaulted(MAX_NUM_CONTROLLER_ANALOG);

	ButtonProperties[0].Key = FRawInputKeys::GenericUSBController_Button1;
	ButtonProperties[1].Key = FRawInputKeys::GenericUSBController_Button2;
	ButtonProperties[2].Key = FRawInputKeys::GenericUSBController_Button3;
	ButtonProperties[3].Key = FRawInputKeys::GenericUSBController_Button4;
	ButtonProperties[4].Key = FRawInputKeys::GenericUSBController_Button5;
	ButtonProperties[5].Key = FRawInputKeys::GenericUSBController_Button6;
	ButtonProperties[6].Key = FRawInputKeys::GenericUSBController_Button7;
	ButtonProperties[7].Key = FRawInputKeys::GenericUSBController_Button8;
	ButtonProperties[8].Key = FRawInputKeys::GenericUSBController_Button9;
	ButtonProperties[9].Key = FRawInputKeys::GenericUSBController_Button10;
	ButtonProperties[10].Key = FRawInputKeys::GenericUSBController_Button11;
	ButtonProperties[11].Key = FRawInputKeys::GenericUSBController_Button12;
	ButtonProperties[12].Key = FRawInputKeys::GenericUSBController_Button13;
	ButtonProperties[13].Key = FRawInputKeys::GenericUSBController_Button14;
	ButtonProperties[14].Key = FRawInputKeys::GenericUSBController_Button15;
	ButtonProperties[15].Key = FRawInputKeys::GenericUSBController_Button16;
	ButtonProperties[16].Key = FRawInputKeys::GenericUSBController_Button17;
	ButtonProperties[17].Key = FRawInputKeys::GenericUSBController_Button18;
	ButtonProperties[18].Key = FRawInputKeys::GenericUSBController_Button19;
	ButtonProperties[19].Key = FRawInputKeys::GenericUSBController_Button20;
	ButtonProperties[20].Key = FRawInputKeys::GenericUSBController_Button21;
	ButtonProperties[21].Key = FRawInputKeys::GenericUSBController_Button22;
	ButtonProperties[22].Key = FRawInputKeys::GenericUSBController_Button23;
	ButtonProperties[23].Key = FRawInputKeys::GenericUSBController_Button24;
	ButtonProperties[24].Key = FRawInputKeys::GenericUSBController_Button25;
	ButtonProperties[25].Key = FRawInputKeys::GenericUSBController_Button26;
	ButtonProperties[26].Key = FRawInputKeys::GenericUSBController_Button27;
	ButtonProperties[27].Key = FRawInputKeys::GenericUSBController_Button28;
	ButtonProperties[28].Key = FRawInputKeys::GenericUSBController_Button29;
	ButtonProperties[29].Key = FRawInputKeys::GenericUSBController_Button30;
	ButtonProperties[30].Key = FRawInputKeys::GenericUSBController_Button31;
	ButtonProperties[31].Key = FRawInputKeys::GenericUSBController_Button32;
	ButtonProperties[32].Key = FRawInputKeys::GenericUSBController_Button33;
	ButtonProperties[33].Key = FRawInputKeys::GenericUSBController_Button34;
	ButtonProperties[34].Key = FRawInputKeys::GenericUSBController_Button35;
	ButtonProperties[35].Key = FRawInputKeys::GenericUSBController_Button36;
	ButtonProperties[36].Key = FRawInputKeys::GenericUSBController_Button37;
	ButtonProperties[37].Key = FRawInputKeys::GenericUSBController_Button38;
	ButtonProperties[38].Key = FRawInputKeys::GenericUSBController_Button39;
	ButtonProperties[39].Key = FRawInputKeys::GenericUSBController_Button40;
	ButtonProperties[40].Key = FRawInputKeys::GenericUSBController_Button41;
	ButtonProperties[41].Key = FRawInputKeys::GenericUSBController_Button42;
	ButtonProperties[42].Key = FRawInputKeys::GenericUSBController_Button43;
	ButtonProperties[43].Key = FRawInputKeys::GenericUSBController_Button44;
	ButtonProperties[44].Key = FRawInputKeys::GenericUSBController_Button45;
	ButtonProperties[45].Key = FRawInputKeys::GenericUSBController_Button46;
	ButtonProperties[46].Key = FRawInputKeys::GenericUSBController_Button47;
	ButtonProperties[47].Key = FRawInputKeys::GenericUSBController_Button48;
	ButtonProperties[48].Key = FRawInputKeys::GenericUSBController_Button49;
	ButtonProperties[49].Key = FRawInputKeys::GenericUSBController_Button50;
	ButtonProperties[50].Key = FRawInputKeys::GenericUSBController_Button51;
	ButtonProperties[51].Key = FRawInputKeys::GenericUSBController_Button52;
	ButtonProperties[52].Key = FRawInputKeys::GenericUSBController_Button53;
	ButtonProperties[53].Key = FRawInputKeys::GenericUSBController_Button54;
	ButtonProperties[54].Key = FRawInputKeys::GenericUSBController_Button55;
	ButtonProperties[55].Key = FRawInputKeys::GenericUSBController_Button56;
	ButtonProperties[56].Key = FRawInputKeys::GenericUSBController_Button57;
	ButtonProperties[57].Key = FRawInputKeys::GenericUSBController_Button58;
	ButtonProperties[58].Key = FRawInputKeys::GenericUSBController_Button59;
	ButtonProperties[59].Key = FRawInputKeys::GenericUSBController_Button60;
	ButtonProperties[60].Key = FRawInputKeys::GenericUSBController_Button61;
	ButtonProperties[61].Key = FRawInputKeys::GenericUSBController_Button62;
	ButtonProperties[62].Key = FRawInputKeys::GenericUSBController_Button63;
	ButtonProperties[63].Key = FRawInputKeys::GenericUSBController_Button64;
	ButtonProperties[64].Key = FRawInputKeys::GenericUSBController_Button65;
	ButtonProperties[65].Key = FRawInputKeys::GenericUSBController_Button66;
	ButtonProperties[66].Key = FRawInputKeys::GenericUSBController_Button67;
	ButtonProperties[67].Key = FRawInputKeys::GenericUSBController_Button68;
	ButtonProperties[68].Key = FRawInputKeys::GenericUSBController_Button69;
	ButtonProperties[69].Key = FRawInputKeys::GenericUSBController_Button70;
	ButtonProperties[70].Key = FRawInputKeys::GenericUSBController_Button71;
	ButtonProperties[71].Key = FRawInputKeys::GenericUSBController_Button72;
	ButtonProperties[72].Key = FRawInputKeys::GenericUSBController_Button73;
	ButtonProperties[73].Key = FRawInputKeys::GenericUSBController_Button74;
	ButtonProperties[74].Key = FRawInputKeys::GenericUSBController_Button75;
	ButtonProperties[75].Key = FRawInputKeys::GenericUSBController_Button76;
	ButtonProperties[76].Key = FRawInputKeys::GenericUSBController_Button77;
	ButtonProperties[77].Key = FRawInputKeys::GenericUSBController_Button78;
	ButtonProperties[78].Key = FRawInputKeys::GenericUSBController_Button79;
	ButtonProperties[79].Key = FRawInputKeys::GenericUSBController_Button80;
	ButtonProperties[80].Key = FRawInputKeys::GenericUSBController_Button81;
	ButtonProperties[81].Key = FRawInputKeys::GenericUSBController_Button82;
	ButtonProperties[82].Key = FRawInputKeys::GenericUSBController_Button83;
	ButtonProperties[83].Key = FRawInputKeys::GenericUSBController_Button84;
	ButtonProperties[84].Key = FRawInputKeys::GenericUSBController_Button85;
	ButtonProperties[85].Key = FRawInputKeys::GenericUSBController_Button86;
	ButtonProperties[86].Key = FRawInputKeys::GenericUSBController_Button87;
	ButtonProperties[87].Key = FRawInputKeys::GenericUSBController_Button88;
	ButtonProperties[88].Key = FRawInputKeys::GenericUSBController_Button89;
	ButtonProperties[89].Key = FRawInputKeys::GenericUSBController_Button90;
	ButtonProperties[90].Key = FRawInputKeys::GenericUSBController_Button91;
	ButtonProperties[91].Key = FRawInputKeys::GenericUSBController_Button92;
	ButtonProperties[92].Key = FRawInputKeys::GenericUSBController_Button93;
	ButtonProperties[93].Key = FRawInputKeys::GenericUSBController_Button94;
	ButtonProperties[94].Key = FRawInputKeys::GenericUSBController_Button95;
	ButtonProperties[95].Key = FRawInputKeys::GenericUSBController_Button96;

	AxisProperties[0].Key = FRawInputKeys::GenericUSBController_Axis1;
	AxisProperties[1].Key = FRawInputKeys::GenericUSBController_Axis2;
	AxisProperties[2].Key = FRawInputKeys::GenericUSBController_Axis3;
	AxisProperties[3].Key = FRawInputKeys::GenericUSBController_Axis4;
	AxisProperties[4].Key = FRawInputKeys::GenericUSBController_Axis5;
	AxisProperties[5].Key = FRawInputKeys::GenericUSBController_Axis6;
	AxisProperties[6].Key = FRawInputKeys::GenericUSBController_Axis7;
	AxisProperties[7].Key = FRawInputKeys::GenericUSBController_Axis8;
	AxisProperties[8].Key = FRawInputKeys::GenericUSBController_Axis9;
	AxisProperties[9].Key = FRawInputKeys::GenericUSBController_Axis10;
	AxisProperties[10].Key = FRawInputKeys::GenericUSBController_Axis11;
	AxisProperties[11].Key = FRawInputKeys::GenericUSBController_Axis12;
	AxisProperties[12].Key = FRawInputKeys::GenericUSBController_Axis13;
	AxisProperties[13].Key = FRawInputKeys::GenericUSBController_Axis14;
	AxisProperties[14].Key = FRawInputKeys::GenericUSBController_Axis15;
	AxisProperties[15].Key = FRawInputKeys::GenericUSBController_Axis16;
	AxisProperties[16].Key = FRawInputKeys::GenericUSBController_Axis17;
	AxisProperties[17].Key = FRawInputKeys::GenericUSBController_Axis18;
	AxisProperties[18].Key = FRawInputKeys::GenericUSBController_Axis19;
	AxisProperties[19].Key = FRawInputKeys::GenericUSBController_Axis20;
	AxisProperties[20].Key = FRawInputKeys::GenericUSBController_Axis21;
	AxisProperties[21].Key = FRawInputKeys::GenericUSBController_Axis22;
	AxisProperties[22].Key = FRawInputKeys::GenericUSBController_Axis23;
	AxisProperties[23].Key = FRawInputKeys::GenericUSBController_Axis24;
}

FName URawInputSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR
FText URawInputSettings::GetSectionText() const
{
	return NSLOCTEXT("RawInputPlugin", "RawInputSettingsSection", "Raw Input");
}

void URawInputSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	
#if PLATFORM_WINDOWS
	FRawInputWindows* RawInput = static_cast<FRawInputWindows*>(static_cast<FRawInputPlugin*>(&FRawInputPlugin::Get())->GetRawInputDevice().Get());

	for (const TPair<int32, FRawWindowsDeviceEntry>& RegisteredDevice : RawInput->RegisteredDeviceList)
	{
		const FRawWindowsDeviceEntry& DeviceEntry = RegisteredDevice.Value;
		if (DeviceEntry.bIsConnected)
		{
			RawInput->SetupBindings(RegisteredDevice.Key, false);
		}
	}

#endif
}
#endif