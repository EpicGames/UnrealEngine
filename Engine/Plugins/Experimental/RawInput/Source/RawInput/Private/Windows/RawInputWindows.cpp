// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/RawInputWindows.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "Windows/AllowWindowsPlatformTypes.h"
#include "RawInputSettings.h"
#include "HAL/PlatformProcess.h"
#include "Framework/Application/SlateApplication.h"
#include "Engine/Engine.h"
#include "Engine/Canvas.h"
#include "GameFramework/HUD.h"

DEFINE_LOG_CATEGORY_STATIC(LogRawInputWindows, Log, All);

FDLLPointers::FDLLPointers()
	: HIDDLLHandle(nullptr)
	, HidD_GetSerialNumberString(nullptr)
	, HidD_GetManufacturerString(nullptr)
	, HidD_GetProductString(nullptr)
	, HidP_GetButtonCaps(nullptr)
	, HidP_GetValueCaps(nullptr)
	, HidP_GetCaps(nullptr)
	, HidP_GetUsages(nullptr)
	, HidP_GetUsageValue(nullptr)
{
}

FDLLPointers::~FDLLPointers()
{
	if (HIDDLLHandle)
	{
		FPlatformProcess::FreeDllHandle(HIDDLLHandle);
		HIDDLLHandle = nullptr;
	}
}

bool FDLLPointers::InitFuncPointers()
{
	HIDDLLHandle = FPlatformProcess::GetDllHandle(TEXT("HID.dll"));
	if (HIDDLLHandle)
	{
		HidP_GetCaps = (FDLLPointers::HidP_GetCapsType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidP_GetCaps"));
		check(HidP_GetCaps);
		HidD_GetSerialNumberString = (FDLLPointers::HidD_GetSerialNumberStringType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidD_GetSerialNumberString"));
		check(HidD_GetSerialNumberString);
		HidD_GetManufacturerString = (FDLLPointers::HidD_GetManufacturerStringType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidD_GetManufacturerString"));
		check(HidD_GetManufacturerString);
		HidD_GetProductString = (FDLLPointers::HidD_GetProductStringType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidD_GetProductString"));
		check(HidD_GetProductString);

		HidP_GetButtonCaps = (FDLLPointers::HidP_GetButtonCapsType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidP_GetButtonCaps"));
		check(HidP_GetButtonCaps);		
		HidP_GetValueCaps = (FDLLPointers::HidP_GetValueCapsType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidP_GetValueCaps"));
		check(HidP_GetValueCaps);
		HidP_GetUsages = (FDLLPointers::HidP_GetUsagesType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidP_GetUsages"));
		check(HidP_GetUsages);
		HidP_GetUsageValue = (FDLLPointers::HidP_GetUsageValueType)FPlatformProcess::GetDllExport(HIDDLLHandle, TEXT("HidP_GetUsageValue"));
		check(HidP_GetUsageValue);
	}

	return (HIDDLLHandle != nullptr);
}


FRawWindowsDeviceEntry::FRawWindowsDeviceEntry()
	: bNeedsUpdate(false)
	, bIsConnected(false)
{
	InitializeNameArrays();
}

FRawWindowsDeviceEntry::FRawWindowsDeviceEntry(const FRawInputRegisteredDevice& InDeviceData)
	: DeviceData(InDeviceData)
{
	InitializeNameArrays();
}

void FRawWindowsDeviceEntry::InitializeNameArrays()
{
	ButtonData.AddDefaulted(MAX_NUM_CONTROLLER_BUTTONS);
	AnalogData.AddDefaulted(MAX_NUM_CONTROLLER_ANALOG);
}

FRawInputWindows::FRawInputWindows(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
	: IRawInput(InMessageHandler)
{
	DefaultDeviceHandle = INDEX_NONE;
	DLLPointers.InitFuncPointers();

	FWindowsApplication* WindowsApplication = (FWindowsApplication*)FSlateApplication::Get().GetPlatformApplication().Get();
	check(WindowsApplication);

	WindowsApplication->AddMessageHandler(*this);
	QueryConnectedDevices();

	// Register a default device if desired
	if (GetDefault<URawInputSettings>()->bRegisterDefaultDevice)
	{
		const uint32 Flags = 0;
		const int32 PageID = 0x01;
		int32 DeviceID = 0x04;
		
		DefaultDeviceHandle = RegisterInputDevice(RIM_TYPEHID, Flags, DeviceID, PageID, nullptr);

		if (DefaultDeviceHandle == INDEX_NONE)
		{
			DeviceID = 0x05;
			DefaultDeviceHandle = RegisterInputDevice(RIM_TYPEHID, Flags, DeviceID, PageID, nullptr);
		}
	}

	AHUD::OnShowDebugInfo.AddRaw(this, &FRawInputWindows::ShowDebugInfo);
}	

FRawInputWindows::~FRawInputWindows()
{
	if (DefaultDeviceHandle != INDEX_NONE)
	{
		RemoveRegisteredInputDevice(DefaultDeviceHandle);
	}
}

void FRawInputWindows::SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	MessageHandler = InMessageHandler;
}

HWND GetWindowHandle()
{
	HWND hWnd = nullptr;

	FWindowsApplication* WindowsApplication = (FWindowsApplication*)FSlateApplication::Get().GetPlatformApplication().Get();
	check(WindowsApplication);

	const TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().GetActiveTopLevelWindow();
	if (ParentWindow.IsValid()  && (ParentWindow->GetNativeWindow().IsValid()))
	{
		hWnd = (HWND)ParentWindow->GetNativeWindow()->GetOSWindowHandle();
	}

	return hWnd;
}

int32 FRawInputWindows::RegisterInputDevice(const int32 DeviceType, const int32 Flags, const uint16 DeviceID, const int16 PageID, const HANDLE Handle)
{
	int32 DeviceHandle = INDEX_NONE;
	FRawInputRegisteredDevice DeviceData(DeviceType, DeviceID, PageID, Handle);
	
	RAWINPUTDEVICE RawInputDevice;

	RawInputDevice.usUsagePage = PageID;
	RawInputDevice.usUsage = DeviceID;
	RawInputDevice.dwFlags = Flags;
	
	// Process input for just the window that requested it.  NOTE: If we pass NULL here events are routed to the window with keyboard focus
	// which is not always known at the HWND level with Slate
	RawInputDevice.hwndTarget = GetWindowHandle();

	// Register the raw input device
	const BOOL bResult = RegisterRawInputDevices(&RawInputDevice, 1, sizeof(RAWINPUTDEVICE));
	if (bResult == FALSE)
	{
		const DWORD LastErrorCode = GetLastError();
		UE_LOGF(LogRawInputWindows, Warning, "Error registering device %d (%d). Code %d", DeviceID, PageID, LastErrorCode);
	}
	else
	{
		QueryConnectedDevices();
		// If this doesn't already exist in our internal list add it
		DeviceHandle = FindRegisteredDeviceHandle(DeviceData);
		if (DeviceHandle == INDEX_NONE)
		{
			// Register all devices of the type specified in the param
			// NOTE: This it kind of a hack. A true refactor of the RawInputPlugin is required but this will make it usable with multiple complementary devices. 
			bool bWasConnected = false;
			for (const FConnectedDeviceInfo& ConnectedDeviceInfo : ConnectedDeviceInfoList)
			{
				if (CompareDeviceInfo(ConnectedDeviceInfo.RIDDeviceInfo, DeviceData))
				{
					DeviceHandle = GetNextInputHandle();
					RegisteredDeviceList.Add(DeviceHandle, DeviceData);

					FRawWindowsDeviceEntry& RegisteredDeviceInfo = RegisteredDeviceList[DeviceHandle];
					CopyConnectedDeviceInfo(RegisteredDeviceInfo, &ConnectedDeviceInfo);

					UE_LOGF(LogRawInputWindows, Log, "VendorID:%04X ProductID:%04X", RegisteredDeviceInfo.DeviceData.VendorID, RegisteredDeviceInfo.DeviceData.ProductID);

					bWasConnected = true;

					SetupBindings(DeviceHandle, true);

					UE_LOGF(LogRawInputWindows, Log, "Device was registered succesfully and is connected (Usage:%d UsagePage:%d)", DeviceData.Usage, DeviceData.UsagePage);
				}
			}
		}
		else
		{
			UE_LOGF(LogRawInputWindows, Warning, "Device already registered.");
		}
	}
	return DeviceHandle;
}

void FRawInputWindows::RemoveRegisteredInputDevice(const int32 DeviceHandle)
{
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(DeviceHandle);
	if (DeviceEntry)
	{
		RegisteredDeviceList.Remove(DeviceHandle);
		if (DeviceHandle == DefaultDeviceHandle)
		{
			DefaultDeviceHandle = INDEX_NONE;
		}
	}	
}

int32 FRawInputWindows::FindRegisteredDeviceHandle(const FRawInputRegisteredDevice& InDeviceData) const
{
	for (const TPair<int32, FRawWindowsDeviceEntry>& DeviceEntryPair : RegisteredDeviceList)
	{
 		if (DeviceEntryPair.Value.DeviceData == InDeviceData)
 		{
 			return DeviceEntryPair.Key;
 		}
	}
	return INDEX_NONE;
}

void FRawInputWindows::SetupBindings(const int32 DeviceHandle, const bool bApplyDefaults)
{
	 FRawWindowsDeviceEntry& DeviceEntry = RegisteredDeviceList[DeviceHandle];

	const URawInputSettings* RawInputSettings = GetDefault<URawInputSettings>();

	bool bDefaultsSetup = false;

	for (const FRawInputDeviceConfiguration& DeviceConfig : RawInputSettings->DeviceConfigurations)
	{
		const int32 VendorID = FCString::Strtoi(*DeviceConfig.VendorID, nullptr, 16);
		const int32 ProductID = FCString::Strtoi(*DeviceConfig.ProductID, nullptr, 16);

		// If VendorId or ProductId are 0, apply to everything
		if ((VendorID == 0 || VendorID == DeviceEntry.DeviceData.VendorID) && (ProductID == 0 || ProductID == DeviceEntry.DeviceData.ProductID))
		{
			const int32 NumButtons = FMath::Min(DeviceConfig.ButtonProperties.Num(), MAX_NUM_CONTROLLER_BUTTONS);
			DeviceEntry.ButtonData.SetNum(NumButtons);
			for (int32 Index = 0; Index < NumButtons; ++Index)
			{
				const FRawInputDeviceButtonProperties& ButtonProps = DeviceConfig.ButtonProperties[Index];
				DeviceEntry.ButtonData[Index].ButtonName = (ButtonProps.bEnabled ? ButtonProps.Key.GetFName() : NAME_None);
			}

			const int32 NumAnalogAxes = FMath::Min(DeviceConfig.AxisProperties.Num(), MAX_NUM_CONTROLLER_ANALOG);
			DeviceEntry.AnalogData.SetNum(NumAnalogAxes);
			for (int32 Index = 0; Index < NumAnalogAxes; ++Index)
			{
				const FRawInputDeviceAxisProperties& AxisProps = DeviceConfig.AxisProperties[Index];
				FAnalogData& AnalogData = DeviceEntry.AnalogData[Index];
				if (AxisProps.bEnabled)
				{
					AnalogData.KeyName = AxisProps.Key.GetFName();
					AnalogData.Offset = AxisProps.Offset;
					AnalogData.bInverted = AxisProps.bInverted;
					AnalogData.bGamepadStick = AxisProps.bGamepadStick;
				}
				else
				{
					AnalogData.KeyName = NAME_None;
				}
			}

			bDefaultsSetup = true;
			break;
		}
	}

	if (!bDefaultsSetup && bApplyDefaults)
	{
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis1, 0);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis2, 1);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis3, 2);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis4, 3);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis5, 4);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis6, 5);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis7, 6);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis8, 7);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis9, 8);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis10, 9);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis11, 10);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis12, 11);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis13, 12);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis14, 13);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis15, 14);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis16, 15);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis17, 16);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis18, 17);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis19, 18);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis20, 19);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis21, 20);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis22, 21);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis23, 22);
		BindAnalogForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Axis24, 23);

		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button1, 0);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button2, 1);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button3, 2);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button4, 3);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button5, 4);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button6, 5);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button7, 6);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button8, 7);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button9, 8);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button10, 9);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button11, 10);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button12, 11);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button13, 12);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button14, 13);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button15, 14);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button16, 15);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button17, 16);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button18, 17);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button19, 18);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button20, 19);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button21, 20);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button22, 21);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button23, 22);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button24, 23);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button25, 24);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button26, 25);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button27, 26);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button28, 27);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button29, 28);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button30, 29);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button31, 30);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button32, 31);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button33, 32);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button34, 33);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button35, 34);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button36, 35);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button37, 36);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button38, 37);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button39, 38);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button40, 39);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button41, 40);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button42, 41);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button43, 42);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button44, 43);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button45, 44);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button46, 45);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button47, 46);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button48, 47);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button49, 48);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button50, 49);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button51, 50);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button52, 51);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button53, 52);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button54, 53);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button55, 54);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button56, 55);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button57, 56);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button58, 57);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button59, 58);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button60, 59);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button61, 60);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button62, 61);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button63, 62);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button64, 63);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button65, 64);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button66, 65);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button67, 66);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button68, 67);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button69, 68);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button70, 69);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button71, 70);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button72, 71);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button73, 72);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button74, 73);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button75, 74);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button76, 75);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button77, 76);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button78, 77);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button79, 78);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button80, 79);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button81, 80);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button82, 81);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button83, 82);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button84, 83);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button85, 84);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button86, 85);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button87, 86);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button88, 87);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button89, 88);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button90, 89);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button91, 90);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button92, 91);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button93, 92);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button94, 93);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button95, 94);
		BindButtonForDevice(DeviceHandle, FRawInputKeyNames::GenericUSBController_Button96, 95);
	}
}

void FRawInputWindows::BindButtonForDevice(const int32 DeviceHandle, const FName KeyName, const int32 ButtonIndex )
{
	if (ButtonIndex > MAX_NUM_CONTROLLER_BUTTONS)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid button index: %d", ButtonIndex);
		return;
	}
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(DeviceHandle);
	if (DeviceEntry)
	{
		DeviceEntry->ButtonData[ButtonIndex].ButtonName = KeyName;
	}
	else
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid device handle: %d",DeviceHandle);
	}
}

void FRawInputWindows::BindAnalogForDevice(const int32 DeviceHandle, const FName KeyName, const int32 AxisIndex)
{
	if (AxisIndex > MAX_NUM_CONTROLLER_ANALOG)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid axis index:%d", AxisIndex);
		return;
	}
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(DeviceHandle);
	if (DeviceEntry)
	{
		DeviceEntry->AnalogData[AxisIndex].KeyName = KeyName;
	}
	else
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid device handle:%d", DeviceHandle);
	}
}

void FRawInputWindows::SetAnalogAxisIsInverted(const int32 DeviceHandle, const int32 AxisIndex, const bool bInvert)
{
	if (AxisIndex >= MAX_NUM_CONTROLLER_ANALOG)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid axis index:%d", AxisIndex);
		return;
	}
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(DeviceHandle);
	if (DeviceEntry)
	{
		if (AxisIndex == INDEX_NONE)
		{
			for (FAnalogData& EachData : DeviceEntry->AnalogData)
			{
				EachData.bInverted = bInvert;
			}
		}
		else
		{
			DeviceEntry->AnalogData[AxisIndex].bInverted = bInvert;
		}
	}
	else
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid device handle:%d", DeviceHandle);
	}
}

void FRawInputWindows::SetAnalogAxisOffset(const int32 DeviceHandle, const int32 AxisIndex, const float Offset)
{
	if (AxisIndex >= MAX_NUM_CONTROLLER_ANALOG)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid axis index:%d", AxisIndex);
		return;
	}
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(DeviceHandle);
	if (DeviceEntry)
	{
		if (AxisIndex == INDEX_NONE)
		{
			for (FAnalogData& EachData : DeviceEntry->AnalogData)
			{
				EachData.Offset = Offset;
			}
		}
		else
		{
			DeviceEntry->AnalogData[AxisIndex].Offset = Offset;
		}
	}
	else
	{
		UE_LOGF(LogRawInputWindows, Warning, "Invalid device handle:%d", DeviceHandle);
	}
}

FRegisteredDeviceInfo FRawInputWindows::GetDeviceInfo(const int32 DeviceHandle) const
{
	FRawWindowsDeviceEntry DeviceEntry = RegisteredDeviceList[DeviceHandle];
	FRegisteredDeviceInfo DeviceInfo;
	
	DeviceInfo.Handle = DeviceHandle;
	DeviceInfo.DeviceName = DeviceEntry.DeviceData.DeviceName;
	DeviceInfo.VendorID = DeviceEntry.DeviceData.VendorID;
	DeviceInfo.ProductID = DeviceEntry.DeviceData.ProductID;

	return DeviceInfo;
}

bool FRawInputWindows::ProcessMessage(const HWND hwnd, const uint32 Msg, const WPARAM wParam, const LPARAM lParam, int32& OutResult)
{
	bool bHandled = false;
	if (Msg == WM_INPUT)
	{

		uint32 Size = 0;
		::GetRawInputData((HRAWINPUT)lParam, RID_INPUT, nullptr, &Size, sizeof(RAWINPUTHEADER));

		RAWINPUT* RawInputDataBuffer = (RAWINPUT*)FMemory_Alloca(Size);

		if (::GetRawInputData((HRAWINPUT)lParam, RID_INPUT, RawInputDataBuffer, &Size, sizeof(RAWINPUTHEADER)) == Size)
		{
			// If we have a delegate, pass the raw data and size to it first. If it returns true it has done something with it and we're done
			if (DataReceivedHandler.IsBound())
			{
				if (DataReceivedHandler.Execute(Size, RawInputDataBuffer))
				{
					return true;
				}
			}

 			bool bIsRegisteredDevice = false;			
			
			// If this is a HID device we need to do some stuff to determine if its one we care about (IE one we registered)
			if (RawInputDataBuffer->header.dwType == RIM_TYPEHID)
			{
				HIDP_CAPS Caps;
				FRawInputRegisteredDevice DeviceData;

				// First we need to get the pre-parsed data
				uint32 BufferSize;

				if (::GetRawInputDeviceInfo(RawInputDataBuffer->header.hDevice, RIDI_PREPARSEDDATA, nullptr, &BufferSize) != RAW_INPUT_ERROR)
				{
					PreParsedData.SetNumUninitialized(BufferSize + 1, EAllowShrinking::No);
							
					if (::GetRawInputDeviceInfo(RawInputDataBuffer->header.hDevice, RIDI_PREPARSEDDATA, PreParsedData.GetData(), &BufferSize) != RAW_INPUT_ERROR)
					{
						// now that we have the PP data we need to get the caps, check those and see if this is a device we registered and if it is store it so we can send it
						if (DLLPointers.HidP_GetCaps((PHIDP_PREPARSED_DATA)PreParsedData.GetData(), &Caps) == HIDP_STATUS_SUCCESS)
						{
							DeviceData = FRawInputRegisteredDevice(RawInputDataBuffer->header.dwType, Caps.Usage, Caps.UsagePage, RawInputDataBuffer->header.hDevice);
						}
					}							
				}

				if (DeviceData.bIsValid)
				{
					// Search for a registered device matching details
					for (const TPair<int32, FRawWindowsDeviceEntry>& DeviceEntryPair : RegisteredDeviceList)
					{
						const FRawWindowsDeviceEntry& EachEntry = DeviceEntryPair.Value;
						if (DeviceData == EachEntry.DeviceData)
						{
							if (!EachEntry.bIsConnected)
							{
								// Repoll connection data as this claims to be disconnected
								QueryConnectedDevices();
							}

							bIsRegisteredDevice = true;
							ParseInputData(DeviceEntryPair.Key, RawInputDataBuffer, (PHIDP_PREPARSED_DATA)PreParsedData.GetData(), Caps);
						}
					}
				}
			}
			else
			{
				// Must be a keyboard/mouse, just send the data as we don't really have any detailed info about those to check if we registered them
				bIsRegisteredDevice = true;
			}
			
			if (bIsRegisteredDevice && FilteredInputDataHandler.IsBound())
			{			
				bHandled = FilteredInputDataHandler.Execute(Size, RawInputDataBuffer);
			}
		}
	}
	return bHandled;
}

void FRawInputWindows::ShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_RawInput("RawInput");
	if (Canvas && HUD->ShouldDisplayDebug(NAME_RawInput))
	{
		FDisplayDebugManager& DisplayDebugManager = Canvas->DisplayDebugManager;
		DisplayDebugManager.SetFont(GEngine->GetSmallFont());
		DisplayDebugManager.SetDrawColor(FColor::Yellow);
		DisplayDebugManager.DrawString(TEXT("RAW INPUT"));

		DisplayDebugManager.SetDrawColor(FColor::White);

		for (const TPair<int32, FRawWindowsDeviceEntry>& DevicePair : RegisteredDeviceList)
		{
			const FRawWindowsDeviceEntry& DeviceEntry = DevicePair.Value;

			DisplayDebugManager.DrawString(FString::Printf(TEXT("Device: %d"), DeviceEntry.DeviceData.DeviceType));
			for (int32 Index = 0; Index < DeviceEntry.ButtonData.Num(); ++Index)
			{
				const FButtonData& DeviceButtonData = DeviceEntry.ButtonData[Index];
				if (!DeviceButtonData.ButtonName.IsNone())
				{
					DisplayDebugManager.DrawString(FString::Printf(TEXT("Button: %s (%d) Val: %s"), *DeviceButtonData.ButtonName.ToString(), Index, (DeviceButtonData.bButtonState ? TEXT("TRUE") : TEXT("FALSE"))));
				}
			}

			for (const FAnalogData& AnalogData : DeviceEntry.AnalogData)
			{
				if (!AnalogData.KeyName.IsNone())
				{
					if (AnalogData.HasValue())
					{
						DisplayDebugManager.DrawString(FString::Printf(TEXT("Analog ID: %s (%d) Val: %f"), *AnalogData.KeyName.ToString(), AnalogData.Index, AnalogData.GetValue()));
					}
					else
					{
						DisplayDebugManager.DrawString(FString::Printf(TEXT("Analog ID: %s (%d) Val: --"), *AnalogData.KeyName.ToString(), AnalogData.Index));
					}
				}
			}
		}
	}
}

void FRawInputWindows::ParseInputData(const int32 InHandle, const RAWINPUT* InRawInputDataBuffer, PHIDP_PREPARSED_DATA InPreParsedData, const HIDP_CAPS& InCapabilities)
{
	FRawWindowsDeviceEntry* DeviceEntry = RegisteredDeviceList.Find(InHandle);
	if (DeviceEntry == nullptr )
	{
		return;
	}
	DeviceEntry->bNeedsUpdate = false;
	
	int32 HIDStatus;

	// buttons
	uint16 NumButtonCaps = InCapabilities.NumberInputButtonCaps;
		
	const uint32 ButtonCapsSize = NumButtonCaps * sizeof(HIDP_BUTTON_CAPS);
	HIDP_BUTTON_CAPS* ButtonCapsBuffer = (HIDP_BUTTON_CAPS*)FMemory_Alloca(ButtonCapsSize);
	FMemory::Memzero(ButtonCapsBuffer, ButtonCapsSize);

	HIDStatus = DLLPointers.HidP_GetButtonCaps(HIDP_REPORT_TYPE::HidP_Input, ButtonCapsBuffer, &NumButtonCaps, InPreParsedData);
		
	if (HIDStatus != HIDP_STATUS_SUCCESS)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Failed to read button caps: %x:%ls", (int32)HIDStatus, *GetErrorString(HIDStatus));
	}
	else
	{
		const int32 NumberOfButtons = ButtonCapsBuffer[0].Range.UsageMax - ButtonCapsBuffer[0].Range.UsageMin + 1;		
		const uint32 ButtonDataBufferSize = NumberOfButtons * sizeof(uint16);
		uint16* ButtonDataBuffer = (uint16*)FMemory_Alloca(ButtonDataBufferSize);
		FMemory::Memzero(ButtonDataBuffer, ButtonDataBufferSize);

		uint32 UsageNumButtonCaps = NumberOfButtons;

		HIDStatus = DLLPointers.HidP_GetUsages(HIDP_REPORT_TYPE::HidP_Input, ButtonCapsBuffer[0].UsagePage, 0, ButtonDataBuffer, &UsageNumButtonCaps, InPreParsedData, (PCHAR)InRawInputDataBuffer->data.hid.bRawData, InRawInputDataBuffer->data.hid.dwSizeHid);
		if (HIDStatus != HIDP_STATUS_SUCCESS)
		{
			UE_LOGF(LogRawInputWindows, Warning, "Failed to read button data: %x:%ls", (int32)HIDStatus, *GetErrorString(HIDStatus));
		}
		else
		{
			for (FButtonData& DeviceButtonData : DeviceEntry->ButtonData)
			{
				// We don't reset previous button state so we can detect state transitions, there's no guarantee this will get called after input has consumed the previous state
				DeviceButtonData.bButtonState = false;
			}

			// The number of pressed buttons will be returned in UsageNumButtonCaps - we reset our struct before we started this so only need to set those
			for (uint32 iButton = 0; iButton < UsageNumButtonCaps; ++iButton)
			{
				const uint16 Index = ButtonDataBuffer[iButton] - ButtonCapsBuffer[0].Range.UsageMin;
				if (Index < DeviceEntry->ButtonData.Num())
				{
					DeviceEntry->ButtonData[Index].bButtonState = true;						
				}
			}
			DeviceEntry->bNeedsUpdate = true;
		}			
	}
	
	// Now the analog entries
	uint16 NumValueCaps = InCapabilities.NumberInputValueCaps;

	const uint32 ValueCapsSize = NumValueCaps * sizeof(HIDP_VALUE_CAPS);
	HIDP_VALUE_CAPS* ValueCapsBuffer = (HIDP_VALUE_CAPS*)FMemory_Alloca(ValueCapsSize);
	checkSlow(ValueCapsBuffer);

	FMemory::Memzero(ValueCapsBuffer, ValueCapsSize);

	HIDStatus = DLLPointers.HidP_GetValueCaps(HIDP_REPORT_TYPE::HidP_Input, ValueCapsBuffer, &NumValueCaps, InPreParsedData);
	if (HIDStatus != HIDP_STATUS_SUCCESS)
	{
		UE_LOGF(LogRawInputWindows, Warning, "Failed to read value caps: %x:%ls", (int32)HIDStatus, *GetErrorString(HIDStatus));
	}
	else
	{
		for (uint16 iValue = 0; iValue < NumValueCaps; ++iValue)
		{
			if (ValueCapsBuffer[iValue].UsagePage == DeviceEntry->DeviceData.UsagePage)
			{
				uint32 EachValue;
				HIDStatus = DLLPointers.HidP_GetUsageValue(HIDP_REPORT_TYPE::HidP_Input, ValueCapsBuffer[iValue].UsagePage, 0, ValueCapsBuffer[iValue].Range.UsageMin, &EachValue, InPreParsedData, (PCHAR)InRawInputDataBuffer->data.hid.bRawData, InRawInputDataBuffer->data.hid.dwSizeHid);
				if (HIDStatus != HIDP_STATUS_SUCCESS)
				{
					UE_LOGF(LogRawInputWindows, Warning, "Failed to read value %d. %x:%ls", iValue, (int32)HIDStatus, *GetErrorString(HIDStatus));
				}
				else
				{
					if (iValue < DeviceEntry->AnalogData.Num())
					{
						if (DeviceEntry->AnalogData[iValue].RangeMin == -1.f)
						{
							DeviceEntry->AnalogData[iValue].RangeMin = ValueCapsBuffer[iValue].LogicalMin;
						}
						if (DeviceEntry->AnalogData[iValue].RangeMax == -1.f)
						{
							DeviceEntry->AnalogData[iValue].RangeMax = ValueCapsBuffer[iValue].LogicalMax;
							if (DeviceEntry->AnalogData[iValue].RangeMax < DeviceEntry->AnalogData[iValue].RangeMin)
							{
								// Need to mask against BitSize, xinput devices return 0xffffffff when they really mean 0x0000ffff
								LONG BitMask = (1 << ValueCapsBuffer[iValue].BitSize) - 1;
								DeviceEntry->AnalogData[iValue].RangeMax = (ValueCapsBuffer[iValue].LogicalMax & BitMask);
							}
						}
						DeviceEntry->AnalogData[iValue].Index = ValueCapsBuffer[iValue].Range.UsageMin;
						DeviceEntry->AnalogData[iValue].Value = (float)EachValue;
						DeviceEntry->bNeedsUpdate = true;
					}
				}
			}
		}
	}
}

void FRawInputWindows::QueryConnectedDevices()
{
	ConnectedDeviceInfoList.Reset();
	
	TArray<RAWINPUTDEVICELIST> DeviceList;
	UINT DeviceCount = 0;

	GetRawInputDeviceList(nullptr, &DeviceCount, sizeof(RAWINPUTDEVICELIST));
	if (DeviceCount == 0)
	{
		return;
	}

	DeviceList.AddUninitialized(DeviceCount);
	GetRawInputDeviceList(DeviceList.GetData(), &DeviceCount, sizeof(RAWINPUTDEVICELIST));
	
	for (const RAWINPUTDEVICELIST& Device : DeviceList)
	{
		uint32 NameLen = 0;
		//Force the use of ANSI versions of these calls
		if (GetRawInputDeviceInfoA(Device.hDevice, RIDI_DEVICENAME, nullptr, &NameLen) == RAW_INPUT_ERROR)
		{
			UE_LOGF(LogRawInputWindows, Warning, "Error reading device name length");
			continue;
		}

		DeviceNameBuffer.SetNumUninitialized(NameLen + 1, EAllowShrinking::No);

		if (GetRawInputDeviceInfoA(Device.hDevice, RIDI_DEVICENAME, DeviceNameBuffer.GetData(), &NameLen) == RAW_INPUT_ERROR)
		{
			const DWORD LastErrorCode = GetLastError();
			if (LastErrorCode != ERROR_FILE_NOT_FOUND)
			{
				UE_LOGF(LogRawInputWindows, Warning, "Error reading device name (GetLastError = %d)", LastErrorCode);
			}
			continue;
		}

		DeviceNameBuffer[NameLen] = 0;
		FString DeviceName = ANSI_TO_TCHAR(DeviceNameBuffer.GetData());
		DeviceName.ReplaceInline(TEXT("#"), TEXT("\\"), ESearchCase::CaseSensitive);
		
		UE_LOGF(LogRawInputWindows, Verbose, "Found device %ls", *DeviceName);

		RID_DEVICE_INFO RawDeviceInfo = { 0 };
		UINT DeviceInfoLen = 0;
		if (GetRawInputDeviceInfoA(Device.hDevice, RIDI_DEVICEINFO, nullptr, &DeviceInfoLen) == RAW_INPUT_ERROR)
		{
			UE_LOGF(LogRawInputWindows, Warning, "Error reading device info size for %ls", *DeviceName);
			continue;
		}

		if (DeviceInfoLen != sizeof(RawDeviceInfo))
		{
			UE_LOGF(LogRawInputWindows, Warning, "Device info size mismatch. Expected for %zu but was actually %d", sizeof( RawDeviceInfo), DeviceInfoLen);
			continue;
		}

		if (GetRawInputDeviceInfo(Device.hDevice,RIDI_DEVICEINFO, &RawDeviceInfo, &DeviceInfoLen) == RAW_INPUT_ERROR)
		{
			UE_LOGF(LogRawInputWindows, Warning, "Error reading device info for %ls", *DeviceName);
			continue;
		}

		// Add to the list of registered devices
		int32 index = ConnectedDeviceInfoList.Add(FConnectedDeviceInfo(MoveTemp(DeviceName), RawDeviceInfo, Device.hDevice));

		ShowDeviceInfo(ConnectedDeviceInfoList.Last());
	}

	// Update registered device info connected status
	for (TPair<int32, FRawWindowsDeviceEntry>& DeviceEntryPair : RegisteredDeviceList)
	{
		bool bFoundConnected = false;
		for (const FConnectedDeviceInfo& ConnectedDeviceInfo : ConnectedDeviceInfoList)
		{
			if (CompareDeviceInfo(ConnectedDeviceInfo.RIDDeviceInfo, DeviceEntryPair.Value.DeviceData))
			{
				bool bWasConnected = DeviceEntryPair.Value.bIsConnected;

				bFoundConnected = true;
				CopyConnectedDeviceInfo(DeviceEntryPair.Value, &ConnectedDeviceInfo);

				if (!bWasConnected)
				{
					// If this is the first connection, apply bindings now
					SetupBindings(DeviceEntryPair.Key, true);
					
					UE_LOGF(LogRawInputWindows, Log, "Device was connected after registration (Usage:%d UsagePage:%d)", DeviceEntryPair.Value.DeviceData.Usage, DeviceEntryPair.Value.DeviceData.UsagePage);				
				}

				break;
			}
		}

		if (!bFoundConnected)
		{
			CopyConnectedDeviceInfo(DeviceEntryPair.Value, nullptr);
		}
	}

	UE_LOGF(LogRawInputWindows, Log, "QueryConnectedDevices found %d devices", ConnectedDeviceInfoList.Num());
}

FString FRawInputWindows::GetErrorString(const int32 StatusCode) const
{
	switch (StatusCode)
	{
	case HIDP_STATUS_SUCCESS:
		return TEXT("HIDStatusSuccess");

	case HIDP_STATUS_NULL:
		return TEXT("HIDStatusNull");

	case HIDP_STATUS_INVALID_PREPARSED_DATA:
		return TEXT("HIDStatusInvalidPreparsedData");

	case HIDP_STATUS_INVALID_REPORT_TYPE:
		return TEXT("HIDStatusInvalidReportType");

	case HIDP_STATUS_INVALID_REPORT_LENGTH:
		return TEXT("HIDStatusInvalidReportLength");

	case HIDP_STATUS_USAGE_NOT_FOUND:
		return TEXT("HIDStatusUsageNotFound");

	case HIDP_STATUS_VALUE_OUT_OF_RANGE:
		return TEXT("HIDStatusValueOutOfRange");

	case HIDP_STATUS_BAD_LOG_PHY_VALUES:
		return TEXT("HIDStatusBadLogPhyValues");

	case HIDP_STATUS_BUFFER_TOO_SMALL:
		return TEXT("HIDStatusBufferTooSmall");

	case HIDP_STATUS_INTERNAL_ERROR:
		return TEXT("HIDStatusInternalError");

	case HIDP_STATUS_I8042_TRANS_UNKNOWN:
		return TEXT("HIDStatusI8042TransUnknown");

	case HIDP_STATUS_INCOMPATIBLE_REPORT_ID:
		return TEXT("HIDStatusIncompatibleReportID");

	case HIDP_STATUS_NOT_VALUE_ARRAY:
		return TEXT("HIDStatusNotValueArray");

	case HIDP_STATUS_IS_VALUE_ARRAY:
		return TEXT("HIDStatusIsValueArray");

	case HIDP_STATUS_DATA_INDEX_NOT_FOUND:
		return TEXT("HIDStatusDataIndexNotFound");

	case HIDP_STATUS_DATA_INDEX_OUT_OF_RANGE:
		return TEXT("HIDStatusDataIndexOutOfRange");

	case HIDP_STATUS_BUTTON_NOT_PRESSED:
		return TEXT("HIDStatusButtonNotPressed");

	case HIDP_STATUS_REPORT_DOES_NOT_EXIST:
		return TEXT("HIDStatusReportDoesNotExist");

	case HIDP_STATUS_NOT_IMPLEMENTED:
		return TEXT("HIDStatusNotImplemented");

	default:
		return TEXT("Unknown status code");
	}
}

bool FRawInputWindows::CompareDeviceInfo(const RID_DEVICE_INFO& DeviceInfo, const FRawInputRegisteredDevice& OtherInfo)
{
	bool bResult = false;
	if (OtherInfo.bIsValid)
	{
		switch (DeviceInfo.dwType)
		{
		case RIM_TYPEMOUSE:
		case RIM_TYPEKEYBOARD:
			bResult = (DeviceInfo.dwType == OtherInfo.DeviceType);
			break;
		case RIM_TYPEHID:
			bResult = ((DeviceInfo.dwType == OtherInfo.DeviceType) &&
				(DeviceInfo.hid.usUsage == OtherInfo.Usage) &&
				(DeviceInfo.hid.usUsagePage == OtherInfo.UsagePage));
			break;
		}
	}
	return bResult;
}

void FRawInputWindows::CopyConnectedDeviceInfo(FRawWindowsDeviceEntry& RegisteredDevice, const FConnectedDeviceInfo* ConnectedDevice)
{
	if (ConnectedDevice)
	{
		RegisteredDevice.bIsConnected = true;
		RegisteredDevice.DeviceData.DeviceName = ConnectedDevice->DeviceName;
		if (RegisteredDevice.DeviceData.DeviceType == RIM_TYPEHID)
		{
			RegisteredDevice.DeviceData.VendorID = ConnectedDevice->RIDDeviceInfo.hid.dwVendorId;
			RegisteredDevice.DeviceData.ProductID = ConnectedDevice->RIDDeviceInfo.hid.dwProductId;
		}

		RegisteredDevice.DeviceData.hDevice = ConnectedDevice->hDevice;
	}
	else
	{
		RegisteredDevice.bIsConnected = false;
	}
}

void FRawInputWindows::ShowDeviceInfo(const FConnectedDeviceInfo& DeviceInfo) const
{
	UE_LOGF(LogRawInputWindows, Verbose, "%ls", *DeviceInfo.DeviceName);
	UE_LOGF(LogRawInputWindows, Verbose, "Device type %d", DeviceInfo.RIDDeviceInfo.dwType);
	switch( DeviceInfo.RIDDeviceInfo.dwType)
	{
	case RIM_TYPEMOUSE:
		UE_LOGF(LogRawInputWindows, Verbose, "dwId:%d, dwNumberOfButtons:%d, dwSampleRate:%d, fHasHorizontalWheel:%d",
		DeviceInfo.RIDDeviceInfo.mouse.dwId,
		DeviceInfo.RIDDeviceInfo.mouse.dwNumberOfButtons,
		DeviceInfo.RIDDeviceInfo.mouse.dwSampleRate,
		DeviceInfo.RIDDeviceInfo.mouse.fHasHorizontalWheel);
		break;
	case RIM_TYPEKEYBOARD:
		UE_LOGF(LogRawInputWindows, Verbose, "dwType:%d, dwSubType:%d, dwKeyboardMode:%d, dwNumberOfFunctionKeys:%d,dwNumberOfIndicators:%d,dwNumberOfKeysTotal:%d",
		DeviceInfo.RIDDeviceInfo.keyboard.dwType,
		DeviceInfo.RIDDeviceInfo.keyboard.dwSubType,
		DeviceInfo.RIDDeviceInfo.keyboard.dwKeyboardMode,
		DeviceInfo.RIDDeviceInfo.keyboard.dwNumberOfFunctionKeys,
		DeviceInfo.RIDDeviceInfo.keyboard.dwNumberOfIndicators,
		DeviceInfo.RIDDeviceInfo.keyboard.dwNumberOfKeysTotal);
		break;
	case RIM_TYPEHID:
		UE_LOGF(LogRawInputWindows, Verbose, "dwVendorId:%04X, dwProductId:%04X, dwVersionNumber:%d, usUsagePage:%d,usUsage:%d",
		DeviceInfo.RIDDeviceInfo.hid.dwVendorId,
		DeviceInfo.RIDDeviceInfo.hid.dwProductId,
		DeviceInfo.RIDDeviceInfo.hid.dwVersionNumber,
		DeviceInfo.RIDDeviceInfo.hid.usUsagePage,
		DeviceInfo.RIDDeviceInfo.hid.usUsage);
		break;
	}
}

static FName RawInputInterfaceName = FName("RawInput");

void FRawInputWindows::SendControllerEvents()
{
	FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPrimaryPlatformUser();
	FInputDeviceId DeviceId = IPlatformInputDeviceMapper::Get().GetDefaultInputDevice();
	
	for (TPair<int32, FRawWindowsDeviceEntry>& DeviceEntryPair : RegisteredDeviceList)
	{
		FRawWindowsDeviceEntry& DeviceEntry = DeviceEntryPair.Value;
		// This is set to true if we need to send this data again next time
		// e.g if a button is still down or axis has a value (e.g. wheel not in centre)
		if (DeviceEntry.bNeedsUpdate)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(this, RawInputInterfaceName, DeviceEntryPair.Key, DeviceEntry.DeviceData.DeviceName);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

			for (FButtonData& DeviceButtonData : DeviceEntry.ButtonData)
			{
				if (!DeviceButtonData.ButtonName.IsNone())
				{
					// If the state changed, fire a button pressed
					if (DeviceButtonData.bButtonState != DeviceButtonData.bPreviousButtonState)
					{
						if (DeviceButtonData.bButtonState)
						{
							MessageHandler->OnControllerButtonPressed(DeviceButtonData.ButtonName, UserId, DeviceId, false);								
						}
						else
						{
							MessageHandler->OnControllerButtonReleased(DeviceButtonData.ButtonName, UserId, DeviceId, false);
						}
						DeviceButtonData.bPreviousButtonState = DeviceButtonData.bButtonState;
					}
					else if (DeviceButtonData.bButtonState) // state not changed - but is true.. which means it must have been true last time.. so is repeat
					{
						MessageHandler->OnControllerButtonPressed(DeviceButtonData.ButtonName, UserId, DeviceId, true);							
					}
						
				}
			}

			for (const FAnalogData& DeviceAnalogData : DeviceEntry.AnalogData)
			{
				if (!DeviceAnalogData.KeyName.IsNone() && DeviceAnalogData.HasValue())
				{
					MessageHandler->OnControllerAnalog(DeviceAnalogData.KeyName, UserId, DeviceId, DeviceAnalogData.GetValue());
				}
			}
		}
	}	
}
