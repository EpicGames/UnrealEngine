// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"

DECLARE_DELEGATE_RetVal_TwoParams( bool, FRawInputDataDelegate, int32 /*DataSize*/, const struct tagRAWINPUT* /*Data */);

class IRawInput : public IInputDevice
{
public:
	IRawInput(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);

	virtual ~IRawInput() {}

	/** Query connected devices and populate the ConnectedDeviceInfoList */
	virtual void QueryConnectedDevices() {}

	/** Register a device for use. */
	virtual int32 RegisterInputDevice(int32 DeviceType, int32 Flags, uint16 DeviceID, int16 PageID, void* Handle) = 0;
	
	/** Remove a registered device */
	virtual void RemoveRegisteredInputDevice(int32 DeviceHandle) {};
	
	/** 
	 * Register a button on a device to send input events with a given name 
	 * @param DeviceHandle	Device handle to bind to
	 * @param KeyName		Key Name (e.g FRawInputKeyNames::GenericUSBController_Button1)
	 * @param ButtonIndex	Index of the button to bind
	 */
	virtual void BindButtonForDevice(int32 DeviceHandle, FName EventName, int32 ButtonIndex ) { }

	/**
	 * Register axis/analog on a device to send input events with a given name
	 * @param DeviceHandle	Device handle to bind to
	 * @param KeyName		Name of key to emit (e.g FRawInputKeyNames::GenericUSBController_Axis1)
	 * @param AxisIndex		Index of the axis/analog to bind
	 */
	virtual void BindAnalogForDevice(int32 DeviceHandle, FName KeyName, int32 AxisIndex ) { }

	/** Returns the delegate for a data received handler */
	FRawInputDataDelegate& GetDataReceivedHandler() { return DataReceivedHandler; }

	/**
	 * Set whether an axis is inverted
	 * @param DeviceHandle	Handle to the device on which to set the axis inverted.
	 * @param AxisIndex		Index of the axis entry to set. If INDEX_NONE all will be set.
	 * @param bInvert		Whether the axis is to have its value inverted or not
	 */
	virtual void SetAnalogAxisIsInverted(int32 DeviceHandle, int32 AxisIndex, bool bInvert) { }

	/**
	 * Sets the offset of the given axis 
	 * @param DeviceHandle	Handle to the device on which to set the offset
	 * @param AxisIndex		Index of the axis to set. If INDEX_NONE all will be set.
	 * @param Offset		Value to offset the normalized (and optionally inverted) value by
	 */
	virtual void SetAnalogAxisOffset(int32 DeviceHandle, int32 AxisIndex, float Offset) { }

protected:

	/** Delegate to allow for manual parsing of HID data. */
	FRawInputDataDelegate DataReceivedHandler;

	/** Handler to send all messages to. */
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	/** Helper to get next input handle to assign. */
	int32 GetNextInputHandle() { return LastAssignedInputHandle++; }

private:

	int32 LastAssignedInputHandle;
};

class FRawInputPlugin : public IInputDeviceModule
{
	virtual TSharedPtr< class IInputDevice > CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override;

	TSharedPtr< class IRawInput > RawInputDevice;

public:
	TSharedPtr< class IRawInput >& GetRawInputDevice() { return RawInputDevice; }

	virtual void StartupModule() override;

	static inline FRawInputPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<FRawInputPlugin>("RawInput");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("RawInput");
	}
};

