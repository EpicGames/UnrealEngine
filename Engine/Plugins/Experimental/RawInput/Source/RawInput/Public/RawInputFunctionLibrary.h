// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "GenericPlatform/GenericApplicationMessageHandler.h"
#include "RawInputFunctionLibrary.generated.h"

struct RAWINPUT_API FRawInputKeyNames
{
	static const FGamepadKeyNames::Type GenericUSBController_Axis1;
	static const FGamepadKeyNames::Type GenericUSBController_Axis2;
	static const FGamepadKeyNames::Type GenericUSBController_Axis3;
	static const FGamepadKeyNames::Type GenericUSBController_Axis4;
	static const FGamepadKeyNames::Type GenericUSBController_Axis5;
	static const FGamepadKeyNames::Type GenericUSBController_Axis6;
	static const FGamepadKeyNames::Type GenericUSBController_Axis7;
	static const FGamepadKeyNames::Type GenericUSBController_Axis8;
	static const FGamepadKeyNames::Type GenericUSBController_Axis9;
	static const FGamepadKeyNames::Type GenericUSBController_Axis10;
	static const FGamepadKeyNames::Type GenericUSBController_Axis11;
	static const FGamepadKeyNames::Type GenericUSBController_Axis12;
	static const FGamepadKeyNames::Type GenericUSBController_Axis13;
	static const FGamepadKeyNames::Type GenericUSBController_Axis14;
	static const FGamepadKeyNames::Type GenericUSBController_Axis15;
	static const FGamepadKeyNames::Type GenericUSBController_Axis16;
	static const FGamepadKeyNames::Type GenericUSBController_Axis17;
	static const FGamepadKeyNames::Type GenericUSBController_Axis18;
	static const FGamepadKeyNames::Type GenericUSBController_Axis19;
	static const FGamepadKeyNames::Type GenericUSBController_Axis20;
	static const FGamepadKeyNames::Type GenericUSBController_Axis21;
	static const FGamepadKeyNames::Type GenericUSBController_Axis22;
	static const FGamepadKeyNames::Type GenericUSBController_Axis23;
	static const FGamepadKeyNames::Type GenericUSBController_Axis24;

	static const FGamepadKeyNames::Type GenericUSBController_Button1;
	static const FGamepadKeyNames::Type GenericUSBController_Button2;
	static const FGamepadKeyNames::Type GenericUSBController_Button3;
	static const FGamepadKeyNames::Type GenericUSBController_Button4;
	static const FGamepadKeyNames::Type GenericUSBController_Button5;
	static const FGamepadKeyNames::Type GenericUSBController_Button6;
	static const FGamepadKeyNames::Type GenericUSBController_Button7;
	static const FGamepadKeyNames::Type GenericUSBController_Button8;
	static const FGamepadKeyNames::Type GenericUSBController_Button9;
	static const FGamepadKeyNames::Type GenericUSBController_Button10;
	static const FGamepadKeyNames::Type GenericUSBController_Button11;
	static const FGamepadKeyNames::Type GenericUSBController_Button12;
	static const FGamepadKeyNames::Type GenericUSBController_Button13;
	static const FGamepadKeyNames::Type GenericUSBController_Button14;
	static const FGamepadKeyNames::Type GenericUSBController_Button15;
	static const FGamepadKeyNames::Type GenericUSBController_Button16;
	static const FGamepadKeyNames::Type GenericUSBController_Button17;
	static const FGamepadKeyNames::Type GenericUSBController_Button18;
	static const FGamepadKeyNames::Type GenericUSBController_Button19;
	static const FGamepadKeyNames::Type GenericUSBController_Button20;
	static const FGamepadKeyNames::Type GenericUSBController_Button21;
	static const FGamepadKeyNames::Type GenericUSBController_Button22;
	static const FGamepadKeyNames::Type GenericUSBController_Button23;
	static const FGamepadKeyNames::Type GenericUSBController_Button24;
	static const FGamepadKeyNames::Type GenericUSBController_Button25;
	static const FGamepadKeyNames::Type GenericUSBController_Button26;
	static const FGamepadKeyNames::Type GenericUSBController_Button27;
	static const FGamepadKeyNames::Type GenericUSBController_Button28;
	static const FGamepadKeyNames::Type GenericUSBController_Button29;
	static const FGamepadKeyNames::Type GenericUSBController_Button30;
	static const FGamepadKeyNames::Type GenericUSBController_Button31;
	static const FGamepadKeyNames::Type GenericUSBController_Button32;
	static const FGamepadKeyNames::Type GenericUSBController_Button33;
	static const FGamepadKeyNames::Type GenericUSBController_Button34;
	static const FGamepadKeyNames::Type GenericUSBController_Button35;
	static const FGamepadKeyNames::Type GenericUSBController_Button36;
	static const FGamepadKeyNames::Type GenericUSBController_Button37;
	static const FGamepadKeyNames::Type GenericUSBController_Button38;
	static const FGamepadKeyNames::Type GenericUSBController_Button39;
	static const FGamepadKeyNames::Type GenericUSBController_Button40;
	static const FGamepadKeyNames::Type GenericUSBController_Button41;
	static const FGamepadKeyNames::Type GenericUSBController_Button42;
	static const FGamepadKeyNames::Type GenericUSBController_Button43;
	static const FGamepadKeyNames::Type GenericUSBController_Button44;
	static const FGamepadKeyNames::Type GenericUSBController_Button45;
	static const FGamepadKeyNames::Type GenericUSBController_Button46;
	static const FGamepadKeyNames::Type GenericUSBController_Button47;
	static const FGamepadKeyNames::Type GenericUSBController_Button48;
	static const FGamepadKeyNames::Type GenericUSBController_Button49;
	static const FGamepadKeyNames::Type GenericUSBController_Button50;
	static const FGamepadKeyNames::Type GenericUSBController_Button51;
	static const FGamepadKeyNames::Type GenericUSBController_Button52;
	static const FGamepadKeyNames::Type GenericUSBController_Button53;
	static const FGamepadKeyNames::Type GenericUSBController_Button54;
	static const FGamepadKeyNames::Type GenericUSBController_Button55;
	static const FGamepadKeyNames::Type GenericUSBController_Button56;
	static const FGamepadKeyNames::Type GenericUSBController_Button57;
	static const FGamepadKeyNames::Type GenericUSBController_Button58;
	static const FGamepadKeyNames::Type GenericUSBController_Button59;
	static const FGamepadKeyNames::Type GenericUSBController_Button60;
	static const FGamepadKeyNames::Type GenericUSBController_Button61;
	static const FGamepadKeyNames::Type GenericUSBController_Button62;
	static const FGamepadKeyNames::Type GenericUSBController_Button63;
	static const FGamepadKeyNames::Type GenericUSBController_Button64;
	static const FGamepadKeyNames::Type GenericUSBController_Button65;
	static const FGamepadKeyNames::Type GenericUSBController_Button66;
	static const FGamepadKeyNames::Type GenericUSBController_Button67;
	static const FGamepadKeyNames::Type GenericUSBController_Button68;
	static const FGamepadKeyNames::Type GenericUSBController_Button69;
	static const FGamepadKeyNames::Type GenericUSBController_Button70;
	static const FGamepadKeyNames::Type GenericUSBController_Button71;
	static const FGamepadKeyNames::Type GenericUSBController_Button72;
	static const FGamepadKeyNames::Type GenericUSBController_Button73;
	static const FGamepadKeyNames::Type GenericUSBController_Button74;
	static const FGamepadKeyNames::Type GenericUSBController_Button75;
	static const FGamepadKeyNames::Type GenericUSBController_Button76;
	static const FGamepadKeyNames::Type GenericUSBController_Button77;
	static const FGamepadKeyNames::Type GenericUSBController_Button78;
	static const FGamepadKeyNames::Type GenericUSBController_Button79;
	static const FGamepadKeyNames::Type GenericUSBController_Button80;
	static const FGamepadKeyNames::Type GenericUSBController_Button81;
	static const FGamepadKeyNames::Type GenericUSBController_Button82;
	static const FGamepadKeyNames::Type GenericUSBController_Button83;
	static const FGamepadKeyNames::Type GenericUSBController_Button84;
	static const FGamepadKeyNames::Type GenericUSBController_Button85;
	static const FGamepadKeyNames::Type GenericUSBController_Button86;
	static const FGamepadKeyNames::Type GenericUSBController_Button87;
	static const FGamepadKeyNames::Type GenericUSBController_Button88;
	static const FGamepadKeyNames::Type GenericUSBController_Button89;
	static const FGamepadKeyNames::Type GenericUSBController_Button90;
	static const FGamepadKeyNames::Type GenericUSBController_Button91;
	static const FGamepadKeyNames::Type GenericUSBController_Button92;
	static const FGamepadKeyNames::Type GenericUSBController_Button93;
	static const FGamepadKeyNames::Type GenericUSBController_Button94;
	static const FGamepadKeyNames::Type GenericUSBController_Button95;
	static const FGamepadKeyNames::Type GenericUSBController_Button96;
};

struct RAWINPUT_API FRawInputKeys
{
	static const FKey GenericUSBController_Axis1;
	static const FKey GenericUSBController_Axis2;
	static const FKey GenericUSBController_Axis3;
	static const FKey GenericUSBController_Axis4;
	static const FKey GenericUSBController_Axis5;
	static const FKey GenericUSBController_Axis6;
	static const FKey GenericUSBController_Axis7;
	static const FKey GenericUSBController_Axis8;
	static const FKey GenericUSBController_Axis9;
	static const FKey GenericUSBController_Axis10;
	static const FKey GenericUSBController_Axis11;
	static const FKey GenericUSBController_Axis12;
	static const FKey GenericUSBController_Axis13;
	static const FKey GenericUSBController_Axis14;
	static const FKey GenericUSBController_Axis15;
	static const FKey GenericUSBController_Axis16;
	static const FKey GenericUSBController_Axis17;
	static const FKey GenericUSBController_Axis18;
	static const FKey GenericUSBController_Axis19;
	static const FKey GenericUSBController_Axis20;
	static const FKey GenericUSBController_Axis21;
	static const FKey GenericUSBController_Axis22;
	static const FKey GenericUSBController_Axis23;
	static const FKey GenericUSBController_Axis24;

	static const FKey GenericUSBController_Button1;
	static const FKey GenericUSBController_Button2;
	static const FKey GenericUSBController_Button3;
	static const FKey GenericUSBController_Button4;
	static const FKey GenericUSBController_Button5;
	static const FKey GenericUSBController_Button6;
	static const FKey GenericUSBController_Button7;
	static const FKey GenericUSBController_Button8;
	static const FKey GenericUSBController_Button9;
	static const FKey GenericUSBController_Button10;
	static const FKey GenericUSBController_Button11;
	static const FKey GenericUSBController_Button12;
	static const FKey GenericUSBController_Button13;
	static const FKey GenericUSBController_Button14;
	static const FKey GenericUSBController_Button15;
	static const FKey GenericUSBController_Button16;
	static const FKey GenericUSBController_Button17;
	static const FKey GenericUSBController_Button18;
	static const FKey GenericUSBController_Button19;
	static const FKey GenericUSBController_Button20;
	static const FKey GenericUSBController_Button21;
	static const FKey GenericUSBController_Button22;
	static const FKey GenericUSBController_Button23;
	static const FKey GenericUSBController_Button24;
	static const FKey GenericUSBController_Button25;
	static const FKey GenericUSBController_Button26;
	static const FKey GenericUSBController_Button27;
	static const FKey GenericUSBController_Button28;
	static const FKey GenericUSBController_Button29;
	static const FKey GenericUSBController_Button30;
	static const FKey GenericUSBController_Button31;
	static const FKey GenericUSBController_Button32;
	static const FKey GenericUSBController_Button33;
	static const FKey GenericUSBController_Button34;
	static const FKey GenericUSBController_Button35;
	static const FKey GenericUSBController_Button36;
	static const FKey GenericUSBController_Button37;
	static const FKey GenericUSBController_Button38;
	static const FKey GenericUSBController_Button39;
	static const FKey GenericUSBController_Button40;
	static const FKey GenericUSBController_Button41;
	static const FKey GenericUSBController_Button42;
	static const FKey GenericUSBController_Button43;
	static const FKey GenericUSBController_Button44;
	static const FKey GenericUSBController_Button45;
	static const FKey GenericUSBController_Button46;
	static const FKey GenericUSBController_Button47;
	static const FKey GenericUSBController_Button48;
	static const FKey GenericUSBController_Button49;
	static const FKey GenericUSBController_Button50;
	static const FKey GenericUSBController_Button51;
	static const FKey GenericUSBController_Button52;
	static const FKey GenericUSBController_Button53;
	static const FKey GenericUSBController_Button54;
	static const FKey GenericUSBController_Button55;
	static const FKey GenericUSBController_Button56;
	static const FKey GenericUSBController_Button57;
	static const FKey GenericUSBController_Button58;
	static const FKey GenericUSBController_Button59;
	static const FKey GenericUSBController_Button60;
	static const FKey GenericUSBController_Button61;
	static const FKey GenericUSBController_Button62;
	static const FKey GenericUSBController_Button63;
	static const FKey GenericUSBController_Button64;
	static const FKey GenericUSBController_Button65;
	static const FKey GenericUSBController_Button66;
	static const FKey GenericUSBController_Button67;
	static const FKey GenericUSBController_Button68;
	static const FKey GenericUSBController_Button69;
	static const FKey GenericUSBController_Button70;
	static const FKey GenericUSBController_Button71;
	static const FKey GenericUSBController_Button72;
	static const FKey GenericUSBController_Button73;
	static const FKey GenericUSBController_Button74;
	static const FKey GenericUSBController_Button75;
	static const FKey GenericUSBController_Button76;
	static const FKey GenericUSBController_Button77;
	static const FKey GenericUSBController_Button78;
	static const FKey GenericUSBController_Button79;
	static const FKey GenericUSBController_Button80;
	static const FKey GenericUSBController_Button81;
	static const FKey GenericUSBController_Button82;
	static const FKey GenericUSBController_Button83;
	static const FKey GenericUSBController_Button84;
	static const FKey GenericUSBController_Button85;
	static const FKey GenericUSBController_Button86;
	static const FKey GenericUSBController_Button87;
	static const FKey GenericUSBController_Button88;
	static const FKey GenericUSBController_Button89;
	static const FKey GenericUSBController_Button90;
	static const FKey GenericUSBController_Button91;
	static const FKey GenericUSBController_Button92;
	static const FKey GenericUSBController_Button93;
	static const FKey GenericUSBController_Button94;
	static const FKey GenericUSBController_Button95;
	static const FKey GenericUSBController_Button96;

};

USTRUCT(BlueprintType)
struct RAWINPUT_API FRegisteredDeviceInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, Category="RawInput")
	int32 Handle = 0;

	// Integer representation of the vendor ID (e.g. 0xC262 = 49762)
	UPROPERTY(BlueprintReadOnly, Category="RawInput")
	int32 VendorID = 0;

	// Integer representation of the product ID (e.g. 0xC262 = 49762)
	UPROPERTY(BlueprintReadOnly, Category="RawInput")
	int32 ProductID = 0;

	// Driver supplied device name
	UPROPERTY(BlueprintReadOnly, Category="RawInput")
	FString DeviceName;
};

UCLASS()
class RAWINPUT_API URawInputFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintPure, Category="RawInput")
	static TArray<FRegisteredDeviceInfo> GetRegisteredDevices();
};

