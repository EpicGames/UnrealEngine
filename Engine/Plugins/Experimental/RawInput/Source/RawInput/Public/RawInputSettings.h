// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "InputCoreTypes.h"

#include "RawInputSettings.generated.h"

USTRUCT()
struct RAWINPUT_API FRawInputDeviceAxisProperties
{
	GENERATED_BODY()

	FRawInputDeviceAxisProperties()
		: bEnabled(true)
		, bInverted(false)
		, bGamepadStick(false)
		, Offset(0.f)
	{
	}

	/** Whether the axis is enabled. */
	UPROPERTY(EditAnywhere, Category="Axis Properties")
	uint8 bEnabled:1;

	/** Which key to emit the values from this axis to. */
	UPROPERTY(EditAnywhere, Category="Axis Properties", meta=(EditCondition="bEnabled"))
	FKey Key;

	/** Whether the value of the axis as supplied from the driver should be inverted. */
	UPROPERTY(EditAnywhere, Category="Axis Properties", meta=(EditCondition="bEnabled"))
	uint8 bInverted:1;

	/** Whether this is a gamepad stick, which means it should scale from -1 to 1 instead of 0 to 1 */
	UPROPERTY(EditAnywhere, Category="Axis Properties", meta=(EditCondition="bEnabled"))
	uint8 bGamepadStick:1;

	/** The amount to offset the axis value by to get the desired range (note that this offset is applied after the value has been inverted). */
	UPROPERTY(EditAnywhere, Category="Axis Properties", meta=(EditCondition="bEnabled"))
	float Offset;
};

USTRUCT()
struct RAWINPUT_API FRawInputDeviceButtonProperties
{
	GENERATED_BODY()

	FRawInputDeviceButtonProperties()
		: bEnabled(true)
	{
	}

	/** Whether the button is enabled. */
	UPROPERTY(EditAnywhere, Category="Button Properties")
	uint8 bEnabled:1;

	/** Which key to emit the values from this button to. */
	UPROPERTY(EditAnywhere, Category="Button Properties", meta=(EditCondition="bEnabled"))
	FKey Key;
};

USTRUCT()
struct RAWINPUT_API FRawInputDeviceConfiguration
{
	GENERATED_BODY()

	FRawInputDeviceConfiguration();

	/** The hexadecimal vendor ID in string form (e.g. 0x046D). If this is empty, will match all vendors */
	UPROPERTY(EditAnywhere, Category="Device Config")
	FString VendorID;

	/** The hexadecimal product ID in string form (e.g. 0xC262). If this is empty, will match all products */
	UPROPERTY(EditAnywhere, Category="Device Config")
	FString ProductID;

	UPROPERTY(EditAnywhere, Category="Device Config")
	TArray<FRawInputDeviceAxisProperties> AxisProperties;

	UPROPERTY(EditAnywhere, Category="Device Config")
	TArray<FRawInputDeviceButtonProperties> ButtonProperties;
};

UCLASS(config=Input, defaultconfig)
class RAWINPUT_API URawInputSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

#if WITH_EDITOR
	// Begin UObject Interface
	virtual void PostEditChangeChainProperty( struct FPropertyChangedChainEvent& PropertyChangedEvent ) override;
	// End UObject Interface
#endif

	// Begin UDeveloperSettings Interface
	virtual FName GetCategoryName() const override;
#if WITH_EDITOR
	virtual FText GetSectionText() const override;
#endif
	// End UDeveloperSettings Interface

	/** List of bindings to apply based on Vendor and Product Id, these are applied in order. To set a default, add an entry with empty product/vendor id at the end of the array */
	UPROPERTY(config, EditAnywhere, Category="Device Configurations")
	TArray<FRawInputDeviceConfiguration> DeviceConfigurations;

	/** Whether the plugin should try to register a default device to handle generic gamepads and joysticks */
	UPROPERTY(config, EditAnywhere, Category="Device Configurations")
	bool bRegisterDefaultDevice = true;
};

