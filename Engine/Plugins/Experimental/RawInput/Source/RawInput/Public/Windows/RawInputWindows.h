// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "RawInput.h"
#include "Windows/WindowsApplication.h"
#include "RawInputFunctionLibrary.h"

#include "hidsdi.h"

#define MAX_NUM_CONTROLLER_BUTTONS 96
#define MAX_NUM_CONTROLLER_ANALOG 24
#define RAW_INPUT_ERROR (uint32)(-1)

class FRawInputWindows;
class AHUD;
class UCanvas;
class FDebugDisplayInfo;

/* Helper class to provide an interface into the HID API via the DLL */
struct FDLLPointers
{
public:
	
	/* typedefs for the functions we want */
	typedef bool(WINAPI *HidD_GetSerialNumberStringType)(HANDLE device, PVOID buffer, uint32 buffer_len);
	typedef bool(WINAPI *HidD_GetManufacturerStringType)(HANDLE handle, PVOID buffer, uint32 buffer_len);
	typedef bool(WINAPI *HidD_GetProductStringType)(HANDLE handle, PVOID buffer, uint32 buffer_len);
	typedef int32(WINAPI *HidP_GetValueCapsType)(HIDP_REPORT_TYPE ReportType, PHIDP_VALUE_CAPS ValueCaps, uint16* ValueCapsLength, PHIDP_PREPARSED_DATA PreparsedData);
	typedef int32(WINAPI *HidP_GetButtonCapsType)(HIDP_REPORT_TYPE ReportType, PHIDP_BUTTON_CAPS ButtonCaps, uint16* ButtonCapsLength, PHIDP_PREPARSED_DATA PreparsedData);
	typedef int32(WINAPI *HidP_GetCapsType)(PHIDP_PREPARSED_DATA PreparsedData, PHIDP_CAPS Capabilities);
	typedef int32(WINAPI *HidP_GetUsagesType)(HIDP_REPORT_TYPE ReportType, uint16 UsagePage, uint16 LinkCollection, uint16* UsageList, uint32* UsageLength, PHIDP_PREPARSED_DATA PreparsedData, PCHAR Report, uint32 ReportLength);
	typedef int32(WINAPI *HidP_GetUsageValueType)(HIDP_REPORT_TYPE ReportType, uint16 UsagePage, uint16 LinkCollection, uint16 Usage, uint32* UsageValue, PHIDP_PREPARSED_DATA PreparsedData, PCHAR Report, uint32 ReportLength);
	
	FDLLPointers();
	virtual ~FDLLPointers();
	
	/** Initialize the function pointers */
	bool InitFuncPointers();

	/** Handle to the DLL */
	void* HIDDLLHandle;
	
	/** Function pointers to the functions within the DLL. */
	HidD_GetSerialNumberStringType HidD_GetSerialNumberString;
	HidD_GetManufacturerStringType HidD_GetManufacturerString;
	HidD_GetProductStringType HidD_GetProductString;	
	HidP_GetButtonCapsType HidP_GetButtonCaps;
	HidP_GetValueCapsType HidP_GetValueCaps;
	HidP_GetCapsType HidP_GetCaps;
	HidP_GetUsagesType HidP_GetUsages;	
	HidP_GetUsageValueType HidP_GetUsageValue;	
};

struct FRawInputRegisteredDevice
{
	FRawInputRegisteredDevice()
		: bIsValid(false)
	{
	}

	FRawInputRegisteredDevice(const int32 InDeviceType, const uint16 InUsage, const int16 InUsagePage, const HANDLE _handle)
		: hDevice(_handle)
		, VendorID(0)
		, ProductID(0)
		, DeviceType(InDeviceType)
		, Usage(InUsage)
		, UsagePage(InUsagePage)
		, bIsValid(true)
	{
	};
	
	bool operator==(const FRawInputRegisteredDevice& Other) const
	{
		return (bIsValid && 
			 Other.bIsValid &&
			 hDevice == Other.hDevice);
			 /*
			 (DeviceType == Other.DeviceType) &&
			 (Usage == Other.Usage) && 
			 (UsagePage == Other.UsagePage));*/
	};
	
	// Driver supplied device name
	FString DeviceName;
	HANDLE hDevice;

	// Device identifiers
	int32 VendorID;
	int32 ProductID;

	// HIDP device values
	int32 DeviceType;
	uint16 Usage;
	uint16 UsagePage;

	// Whether the data has been populated
	uint8 bIsValid:1;
};

struct FAnalogData
{
	FAnalogData()
		: Index(INDEX_NONE)
		, Value(0.f)
		, PreviousValue(0.f)
		, RangeMin(-1.f)
		, RangeMax(-1.f)
		, Offset(0.f)
		, bInverted(false)
		, bGamepadStick(false)
	{
	}

	FAnalogData(const int32 InIndex, const float InValue, const float InRangeMin, const float InRangeMax, const float InOffset, const bool bInInverted, const bool bInGamepadStick, const FName InKeyName)
		: Index(InIndex)
		, Value(InValue)
		, PreviousValue(0.f)
		, RangeMin(InRangeMin)
		, RangeMax(InRangeMax)
		, Offset(0.f)
		, bInverted(bInInverted)
		, bGamepadStick(bInGamepadStick)
		, KeyName(InKeyName)
	{
	}

	/* Helper function to get the offset and normalized value */
	float GetValue() const
	{
		const float Factor = 1.f / (RangeMax - RangeMin);
		if (bGamepadStick)
		{
			// Need to do the re-centering before inverting
			const float NormalizedValue = ((Value * Factor) - 0.5f) * 2 * (bInverted ? -1.0f : 1.0f);
			return NormalizedValue + Offset;
		}

		const float NormalizedValue = (bInverted ? (Value * Factor * -1.f) : (Value * Factor));
		return NormalizedValue + Offset;
	}

	/* Whether the data represents a valid value */
	bool HasValue() const
	{
		return ((Index != INDEX_NONE) && !KeyName.IsNone() && ((RangeMin != -1.f) || (RangeMax != -1.f)));
	}

	/* Index in the value data*/
	int32 Index;

	/* Current analog value */
	float Value;
	
	/* Last analog value */
	float PreviousValue;

	/* Min Analog value */
	float RangeMin;
	
	/* Max analog value */
	float RangeMax;
	
	/* Offset to apply to normalized axis value */
	float Offset;

	/* Is this axis inverted */
	bool bInverted;

	/* Is this axis centered on 0 instead of 0.5 */
	bool bGamepadStick;
	
	/* Key name */
	FName KeyName;
};

struct FButtonData
{
	FButtonData()
		: bButtonState(false)
		, bPreviousButtonState(false)
	{
	}

	/* Current button state */
	uint8 bButtonState:1;

	/* Button state last update */
	uint8 bPreviousButtonState:1;

	/* Button name */
	FName ButtonName;
};

struct FRawWindowsDeviceEntry
{
public:
	FRawWindowsDeviceEntry();
	FRawWindowsDeviceEntry(const FRawInputRegisteredDevice& InDeviceData);

	void InitializeNameArrays();

	FRawInputRegisteredDevice DeviceData;

	/* Button data */
	TArray<FButtonData> ButtonData;
	
	/* Analog data */
	TArray<FAnalogData>	AnalogData;
	
	/* Device has controller data to send */
	uint8 bNeedsUpdate:1;
	
	/* Device is connected */
	uint8 bIsConnected:1;
};

struct FConnectedDeviceInfo
{
public:
	FConnectedDeviceInfo() {}
	FConnectedDeviceInfo(FString InDeviceName, const RID_DEVICE_INFO& InRIDDeviceInfo, const HANDLE Handle)
		: DeviceName(MoveTemp(InDeviceName))
		, RIDDeviceInfo(InRIDDeviceInfo)
		, hDevice(Handle)
	{
	}

	FString DeviceName;
	RID_DEVICE_INFO RIDDeviceInfo;
	HANDLE hDevice;
};

class FRawInputWindows : public IRawInput, IWindowsMessageHandler
{

public:
	FRawInputWindows(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler);

	virtual ~FRawInputWindows();

	// Begin RawInput interface
	virtual void QueryConnectedDevices() override;	
	virtual int32 RegisterInputDevice(int32 DeviceType, int32 Flags, uint16 DeviceID, int16 PageID, HANDLE Handle) override;
	virtual void RemoveRegisteredInputDevice(int32 DeviceHandle) override;	
	virtual void BindButtonForDevice(int32 DeviceHandle, FName EventName, int32 ButtonIndex ) override;
	virtual void BindAnalogForDevice(int32 DeviceHandle, FName EventName, int32 AxisIndex ) override;
	virtual void SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override;
	virtual void SetAnalogAxisIsInverted(int32 DeviceHandle, int32 AxisIndex, bool bInvert) override;
	virtual void SetAnalogAxisOffset(int32 DeviceHandle, int32 AxisIndex, float Offset) override;
	// End RawInput interface

	// Begin IInputDevice interface
	virtual void Tick(float DeltaTime) override {}
	virtual void SendControllerEvents() override;
	virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override {}
	virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values) override {}
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override { return false; }
	// End IInputDevice interface

	// Begin IWindowsMessageHandler interface
	virtual bool ProcessMessage(HWND hwnd, uint32 msg, WPARAM wParam, LPARAM lParam, int32& OutResult) override;
	// End IWindowsMessageHandler interface
	
	/** Get the delegate for raw data can be handled manually. */
	FRawInputDataDelegate& GetFilteredInputDataHandler() { return FilteredInputDataHandler; }
	
	/** Return a string for the error of a given status code. */
	FString GetErrorString(int32 StatusCode) const;

	/** Returns user facing data for a registered device. */
	FRegisteredDeviceInfo GetDeviceInfo(int32 DeviceHandle) const;

private:

	/** Adds on screen messages for the buttons stats and analog values when "ShowDebug RawInput" has been specified via the console. */
	void ShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	/** Dumps the properties of a RID_DEVICE_INFO struct to the log. */
	void ShowDeviceInfo(const FConnectedDeviceInfo& DeviceInfo) const;
	
	/** 
	 * Sets up the bindings for the specified device. 
	 * @param DeviceHandle		The handle to the device to setup
	 * @param bApplyDefaults	Whether to setup default bindings if no entry is found in the Raw Input Settings device configurations.
	 */
	void SetupBindings(int32 DeviceHandle, bool bApplyDefaults);

	/** Parse raw input data. */
	void ParseInputData(int32 Handle, const RAWINPUT* RawInputData, PHIDP_PREPARSED_DATA Data, const HIDP_CAPS& Capabilities);
	
	/** Compare a RID_DEVICE_INFO structure with a FRawInputRegisteredDevice one. */
	static bool CompareDeviceInfo(const RID_DEVICE_INFO& DeviceInfo, const FRawInputRegisteredDevice& OtherInfo);

	/** Updates a raw device with connected device info, if null will reset data */
	static void CopyConnectedDeviceInfo(FRawWindowsDeviceEntry& RegisteredDevice, const FConnectedDeviceInfo* ConnectedDevice);

	/** Returns the handle to a registered device or INDEX_NONE if the device doesn't exist. */
	int32 FindRegisteredDeviceHandle(const FRawInputRegisteredDevice& InDeviceData) const;

	/** Handler for filtered raw data (i.e. only devices we registered via RegisterInputDevice) */
	FRawInputDataDelegate FilteredInputDataHandler;
	
	/** List of connected devices populated by GetRawInputDeviceList. */
	TArray<FConnectedDeviceInfo> ConnectedDeviceInfoList;
	
	/** Map of device handles to details of registered devices */
	TMap<int32, FRawWindowsDeviceEntry> RegisteredDeviceList;
	
	/** Structure for HID DLL interaction. */
	FDLLPointers DLLPointers;

	/** Handle to the default device. */
	int32 DefaultDeviceHandle;

	/** Private resizable memory blocks for API calls */
	TArray<uint8> PreParsedData;
	TArray<char> DeviceNameBuffer;

	friend class URawInputFunctionLibrary;
	friend class URawInputSettings;
};

typedef FRawInputWindows FPlatformRawInput;
