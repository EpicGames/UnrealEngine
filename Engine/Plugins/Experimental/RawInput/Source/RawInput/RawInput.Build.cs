// Copyright Epic Games, Inc. All Rights Reserved.

using System;

namespace UnrealBuildTool.Rules
{
	public class RawInput : ModuleRules
	{
		public RawInput(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"ApplicationCore",
					"InputCore",
					"SlateCore",
					"Slate"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"InputDevice",
					"DeveloperSettings"
				}
			);
		}
	}
}
